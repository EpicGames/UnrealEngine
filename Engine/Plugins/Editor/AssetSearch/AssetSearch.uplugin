{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Asset Search",
	"Description": "",
	"Category": "Editor",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "AssetSearch",
			"Type": "EditorNoCommandlet",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "SQLiteCore",
			"Enabled": true
		}
	]
}