// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AssetSearch : ModuleRules
{
	public AssetSearch(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"Json",
				"StudioTelemetry"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"SQLiteCore",
				"InputCore",
				"Slate",
				"SlateCore",
				"GameplayTags",
				"WorkspaceMenuStructure",
				"EditorFramework",
				"UnrealEd",
				"AssetRegistry",
				"JsonUtilities",
				"Projects",
				"PropertyPath",
				"UMG",
				"UMGEditor",
				"BlueprintGraph",
				"DeveloperSettings"
			}
		);

		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"DerivedDataCache",
				"MaterialEditor"
			}
		);
	}
}
