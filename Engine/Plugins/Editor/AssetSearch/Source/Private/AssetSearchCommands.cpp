// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetSearchCommands.h"
#include "Styling/AppStyle.h" 

#define LOCTEXT_NAMESPACE "AssetSearchCommands"

FAssetSearchCommands::FAssetSearchCommands() : TCommands<FAssetSearchCommands>(
	"AssetSearchCommands",
	NSLOCTEXT("Contexts", "AssetSearchCommands", "Asset Search"),
	NAME_None, 
	FAppStyle::GetAppStyleSetName())
{
}

void FAssetSearchCommands::RegisterCommands()
{
	UI_COMMAND(ViewAssetSearch, "Search...", "Show Search Tab", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
