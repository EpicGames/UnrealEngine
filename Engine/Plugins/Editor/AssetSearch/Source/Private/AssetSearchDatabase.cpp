// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetSearchDatabase.h"

#include "SQLiteDatabase.h"
#include "HAL/FileManager.h"

#include "AssetRegistry/AssetData.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "Algo/LevenshteinDistance.h"
#include "SQLitePreparedStatement.h"
#include "SearchQuery.h"
#include "Serialization/JsonReader.h"

DEFINE_LOG_CATEGORY(LogAssetSearch);

FString ConvertToFullTextSearchQuery(const FString& QueryText);

enum class EAssetSearchDatabaseVersion
{
	Empty,
	Initial,
	IndexingAssetIdsAssetPathsUnique,
	IntroducingFileHashing,
	MovingFileInfoIntoAnotherDatabase,
	ClassPathIsNowFullPath,

	// -----<new versions can be added above this line>-------------------------------------------------
	VersionPlusOne,
	LatestVersion = VersionPlusOne - 1
};

class FAssetSearchDatabaseStatements
{
private:
	FSQLiteDatabase& Database;

public:
	explicit FAssetSearchDatabaseStatements(FSQLiteDatabase& InDatabase)
		: Database(InDatabase)
	{
		check(Database.IsValid());
	}

	bool CreatePreparedStatements()
	{
		check(Database.IsValid());

#define PREPARE_STATEMENT(VAR)																				\
			(VAR) = Database.PrepareStatement<decltype(VAR)>(ESQLitePreparedStatementFlags::Persistent);	\
			if (!(VAR).IsValid()) { return false; }

		PREPARE_STATEMENT(Statement_BeginTransaction);
		PREPARE_STATEMENT(Statement_CommitTransaction);
		PREPARE_STATEMENT(Statement_RollbackTransaction);

		PREPARE_STATEMENT(Statement_GetAllAssets);
		PREPARE_STATEMENT(Statement_GetAssetIdForAssetPath);
		PREPARE_STATEMENT(Statement_IsAssetUpToDate);
		PREPARE_STATEMENT(Statement_GetTotalSearchRecords);
		PREPARE_STATEMENT(Statement_AddAssetToAssetTable);
		PREPARE_STATEMENT(Statement_AddAssetProperty);
		PREPARE_STATEMENT(Statement_DeleteEntriesForAsset);

		PREPARE_STATEMENT(Statement_SearchAssets);
		PREPARE_STATEMENT(Statement_SearchAssetPropertiesFTS);

#undef PREPARE_STATEMENT

		return true;
	}

	/**
	 * Statements managing database transactions
	 */

	 /** Begin a database transaction */
	SQLITE_PREPARED_STATEMENT_SIMPLE(FBeginTransaction, "BEGIN TRANSACTION;");
	FBeginTransaction Statement_BeginTransaction;
	bool BeginTransaction()
	{
		return Statement_BeginTransaction.Execute();
	}

	/** Commit a database transaction */
	SQLITE_PREPARED_STATEMENT_SIMPLE(FCommitTransaction, "COMMIT TRANSACTION;");
	FCommitTransaction Statement_CommitTransaction;
	bool CommitTransaction()
	{
		return Statement_CommitTransaction.Execute();
	}

	/** Rollback a database transaction */
	SQLITE_PREPARED_STATEMENT_SIMPLE(FRollbackTransaction, "ROLLBACK TRANSACTION;");
	FRollbackTransaction Statement_RollbackTransaction;
	bool RollbackTransaction()
	{
		return Statement_RollbackTransaction.Execute();
	}

	/**
	 * Application Statements
	 */

	SQLITE_PREPARED_STATEMENT(FIsAssetUpToDate,
		"SELECT index_hash FROM table_assets WHERE asset_path = ?1;",
		SQLITE_PREPARED_STATEMENT_COLUMNS(FString),
		SQLITE_PREPARED_STATEMENT_BINDINGS(FString)
	);
	private: FIsAssetUpToDate Statement_IsAssetUpToDate;
	public: bool IsAssetUpToDate(const FAssetData& InAssetData, const FString& IndexedJsonHash)
	{
		FString OutIndexedJsonHash;
		if (Statement_IsAssetUpToDate.BindAndExecuteSingle(InAssetData.GetObjectPathString(), OutIndexedJsonHash))
		{
			return OutIndexedJsonHash.Equals(IndexedJsonHash, ESearchCase::CaseSensitive);
		}

		return false;
	}

	 SQLITE_PREPARED_STATEMENT(FGetTotalSearchRecords,
		"SELECT COUNT(assetid) FROM table_asset_properties;",
		SQLITE_PREPARED_STATEMENT_COLUMNS(int64),
		SQLITE_PREPARED_STATEMENT_BINDINGS()
	);
	private: FGetTotalSearchRecords Statement_GetTotalSearchRecords;
	public: int64 GetTotalSearchRecords()
	{
		int64 OutTotalSearchRecords;
		if (Statement_GetTotalSearchRecords.BindAndExecuteSingle(OutTotalSearchRecords))
		{
			return OutTotalSearchRecords;
		}

		return INDEX_NONE;
	}

	SQLITE_PREPARED_STATEMENT(FGetAssetIdForAssetPath,
		"SELECT assetid FROM table_assets WHERE asset_path = ?1;",
		SQLITE_PREPARED_STATEMENT_COLUMNS(int64),
		SQLITE_PREPARED_STATEMENT_BINDINGS(FString)
	);
	private: FGetAssetIdForAssetPath Statement_GetAssetIdForAssetPath;
	public: int64 GetAssetIdForAsset(const FAssetData& InAssetData)
	{
		int64 OutAssetId = INDEX_NONE;
		if (Statement_GetAssetIdForAssetPath.BindAndExecuteSingle(InAssetData.GetObjectPathString(), OutAssetId))
		{
			return OutAssetId;
		}

		return INDEX_NONE;
	}

	SQLITE_PREPARED_STATEMENT_BINDINGS_ONLY(
		FAddAssetToAssetTable,
		" INSERT INTO table_assets(asset_name, asset_class, asset_path, index_hash)"
		" VALUES(?1, ?2, ?3, ?4);",
		SQLITE_PREPARED_STATEMENT_BINDINGS(FString, FString, FString, FString)
	);
	private: FAddAssetToAssetTable Statement_AddAssetToAssetTable;
	SQLITE_PREPARED_STATEMENT_BINDINGS_ONLY(
		FAddAssetPropertiesFromJson,
		" INSERT INTO table_asset_properties(assetid, object_name, object_path, object_native_class, property_name, property_field, property_class, value_text, value_hidden)"
		" VALUES(?1, ?2, ?3, ?4, ?5, ?6, ?7, ?8, ?9);",
		SQLITE_PREPARED_STATEMENT_BINDINGS(int64, FString, FString, FString, FString, FString, FString, FString, FString)
	);
	private: FAddAssetPropertiesFromJson Statement_AddAssetProperty;
	public: bool AddSearchRecord(const FAssetData& InAssetData, const FString& IndexedJson, const FString& IndexedJsonHash)
	{
		FString AssetObjectPath = InAssetData.GetObjectPathString();
		if (Statement_AddAssetToAssetTable.BindAndExecute(InAssetData.AssetName.ToString(), InAssetData.AssetClassPath.ToString(), AssetObjectPath, IndexedJsonHash))
		{
			int64 AssetId = Database.GetLastInsertRowId();

			int32 EntriesAdded = 0;

			ensure(BeginTransaction());

			TSharedRef<TJsonReader<>> JsonReader = TJsonReaderFactory<>::Create(IndexedJson);

			EJsonNotation Notation;

			ensure(JsonReader->ReadNext(Notation) && Notation == EJsonNotation::ObjectStart);

			ensure(JsonReader->ReadNext(Notation) && Notation == EJsonNotation::Number && JsonReader->GetIdentifier() == TEXT("version"));
			const int32 SerializerVersion = (int32)JsonReader->GetValueAsNumber();

			ensure(JsonReader->ReadNext(Notation) && Notation == EJsonNotation::ObjectStart && JsonReader->GetIdentifier() == TEXT("indexers"));

			while (JsonReader->ReadNext(Notation))
			{
				// Indexer
				if (Notation == EJsonNotation::ObjectStart)
				{
					const FString IndexerName = JsonReader->GetIdentifier();

					ensure(JsonReader->ReadNext(Notation) && Notation == EJsonNotation::Number && JsonReader->GetIdentifier() == TEXT("version"));
					const int32 IndexerVersion = (int32)JsonReader->GetValueAsNumber();

					ensure(JsonReader->ReadNext(Notation) && Notation == EJsonNotation::ArrayStart && JsonReader->GetIdentifier() == TEXT("objects"));

					// Objects[]
					while (JsonReader->ReadNext(Notation) && Notation != EJsonNotation::ArrayEnd)
					{
						if (Notation == EJsonNotation::ObjectStart)
						{
							FString object_name;
							FString object_path;
							FString object_native_class;

							ensure(JsonReader->ReadNext(Notation) && Notation == EJsonNotation::String && JsonReader->GetIdentifier() == TEXT("name"));
							object_name = JsonReader->GetValueAsString();
							ensure(JsonReader->ReadNext(Notation) && Notation == EJsonNotation::String && JsonReader->GetIdentifier() == TEXT("path"));
							object_path = JsonReader->GetValueAsString();
							ensure(JsonReader->ReadNext(Notation) && Notation == EJsonNotation::String && JsonReader->GetIdentifier() == TEXT("native_class"));
							object_native_class = JsonReader->GetValueAsString();

							ensure(JsonReader->ReadNext(Notation) && Notation == EJsonNotation::ArrayStart && JsonReader->GetIdentifier() == TEXT("properties"));

							FString property_name;
							FString property_field;
							FString property_class;
							FString value_text;
							FString value_hidden;

							// Begin Properties[]
							while (JsonReader->ReadNext(Notation) && Notation != EJsonNotation::ArrayEnd)
							{
								if (Notation == EJsonNotation::ObjectStart)
								{
									// Read all attributes of a property
									while (JsonReader->ReadNext(Notation) && Notation != EJsonNotation::ObjectEnd)
									{
										if (Notation == EJsonNotation::String)
										{
											if (JsonReader->GetIdentifier() == TEXT("name"))
											{
												property_name = JsonReader->GetValueAsString();
											}
											else if (JsonReader->GetIdentifier() == TEXT("field"))
											{
												property_field = JsonReader->GetValueAsString();
											}
											else if (JsonReader->GetIdentifier() == TEXT("class"))
											{
												property_class = JsonReader->GetValueAsString();
											}
											else if (JsonReader->GetIdentifier() == TEXT("value_text"))
											{
												value_text = JsonReader->GetValueAsString();
											}
											else if (JsonReader->GetIdentifier() == TEXT("value_hidden"))
											{
												value_hidden = JsonReader->GetValueAsString();
											}
										}
									}

									if (!Statement_AddAssetProperty.BindAndExecute(AssetId, object_name, object_path, object_native_class, property_name, property_field, property_class, value_text, value_hidden))
									{
										UE_LOGF(LogAssetSearch, Error, "Failed to add entry for asset '%ls': %ls", *AssetObjectPath, *Database.GetLastError());
									}
									else
									{
										EntriesAdded++;
									}
								}
							}
							// End Properties[]
						}
					}
					// End Objects[]
				}
			}

			if (!CommitTransaction())
			{
				UE_LOGF(LogAssetSearch, Error, "Failed commit entries for asset '%ls': %ls", *AssetObjectPath, *Database.GetLastError());
				EntriesAdded = 0;
			}

			if (EntriesAdded > 0)
			{
				//UE_LOGF(LogAssetSearch, Log, "Adding '%ls': %ls", *AssetObjectPath, *Database.GetLastError());
			}

			return true;
		}

		return false;
	}

	SQLITE_PREPARED_STATEMENT_BINDINGS_ONLY(FDeleteEntriesForAsset, "DELETE FROM table_assets WHERE asset_path = ?1;", SQLITE_PREPARED_STATEMENT_BINDINGS(FString));
	private: FDeleteEntriesForAsset Statement_DeleteEntriesForAsset;
	public: bool DeleteEntriesForAsset(const FAssetData& InAssetData)
	{
		return DeleteEntriesForAsset(InAssetData.GetObjectPathString());
	}
	public: bool DeleteEntriesForAsset(const FString& InAssetObjectPath)
	{
		if (Statement_DeleteEntriesForAsset.BindAndExecute(InAssetObjectPath))
		{
			return true;
		}
		return false;
	}

	SQLITE_PREPARED_STATEMENT(FSearchAssets,
		" SELECT "
		"     asset_name, "
		"     asset_class, "
		"     asset_path "
		" FROM table_assets "
		" WHERE asset_name LIKE ?1 "
		";",
		SQLITE_PREPARED_STATEMENT_COLUMNS(FString, FString, FString),
		SQLITE_PREPARED_STATEMENT_BINDINGS(FString)
	);
	FSearchAssets Statement_SearchAssets;
	bool SearchAssets(const FString& QueryText, TFunctionRef<ESQLitePreparedStatementExecuteRowResult(FSearchRecord&&)> InCallback)
	{
		const FString Q = TEXT("%") + QueryText + TEXT("%");

		return Statement_SearchAssets.BindAndExecute(Q, [QueryText, &InCallback](const FSearchAssets& InStatement)
		{
			FSearchRecord Result;
			FString ResultClassPath;
			if (InStatement.GetColumnValues(Result.AssetName, ResultClassPath, Result.AssetPath))
			{
				Result.AssetClass.TrySetPath(ResultClassPath);
				const float WorstCase = Result.AssetName.Len() + QueryText.Len();
				Result.Score = -50.0f * (1.0f - (Algo::LevenshteinDistance(Result.AssetName.ToLower(), QueryText.ToLower()) / WorstCase));
				
				return InCallback(MoveTemp(Result));
			}
			return ESQLitePreparedStatementExecuteRowResult::Error;
		}) != INDEX_NONE;
	}

	SQLITE_PREPARED_STATEMENT(FSearchAssetPropertiesFTS,
		" SELECT "
		"     asset_name, "
		"     asset_class, "
		"     asset_path, "
		"     object_name, "
		"     object_path, "
		"     object_native_class, "
		"     property_name, "
		"     property_field, "
		"     property_class, "
		"     value_text, "
		"     value_hidden, "
		"     rank as score "
		" FROM table_asset_properties_fts "
		" WHERE table_asset_properties_fts MATCH ?1 "
		";",
		SQLITE_PREPARED_STATEMENT_COLUMNS(FString, FString, FString, FString, FString, FString, FString, FString, FString, FString, FString, float),
		SQLITE_PREPARED_STATEMENT_BINDINGS(FString)
	);
	FSearchAssetPropertiesFTS Statement_SearchAssetPropertiesFTS;
	bool SearchAssetProperties(const FString& QueryText, TFunctionRef<ESQLitePreparedStatementExecuteRowResult(FSearchRecord&&)> InCallback)
	{
		const FString Q = ConvertToFullTextSearchQuery(QueryText);

		return Statement_SearchAssetPropertiesFTS.BindAndExecute(Q, [&InCallback](const FSearchAssetPropertiesFTS& InStatement)
		{
			FSearchRecord Result;
			FString ResultClassPath;
			if (InStatement.GetColumnValues(
				Result.AssetName, ResultClassPath, Result.AssetPath,
				Result.object_name, Result.object_path, Result.object_native_class,
				Result.property_name, Result.property_field, Result.property_class,
				Result.value_text, Result.value_hidden,
				Result.Score))
			{
				Result.AssetClass.TrySetPath(ResultClassPath);
				return InCallback(MoveTemp(Result));
			}
			return ESQLitePreparedStatementExecuteRowResult::Error;
		}) != INDEX_NONE;
	}

	SQLITE_PREPARED_STATEMENT(FGetAllAssets,
		"SELECT asset_path FROM table_assets;",
		SQLITE_PREPARED_STATEMENT_COLUMNS(FString),
		SQLITE_PREPARED_STATEMENT_BINDINGS()
	);
	FGetAllAssets Statement_GetAllAssets;
	bool GetAllAssets(TFunctionRef<ESQLitePreparedStatementExecuteRowResult(FString&&)> InCallback)
	{
		return Statement_GetAllAssets.BindAndExecute([&InCallback](const FGetAllAssets& InStatement) //-V562
		{
			FString AssetPath;
			if (InStatement.GetColumnValues(AssetPath))
			{
				return InCallback(MoveTemp(AssetPath));
			}
			return ESQLitePreparedStatementExecuteRowResult::Error;
		}) != INDEX_NONE;
	}
};

class FAssetSearchDatabaseScopedTransaction
{
public:
	explicit FAssetSearchDatabaseScopedTransaction(FAssetSearchDatabaseStatements& InStatements)
		: Statements(InStatements)
		, bHasTransaction(Statements.BeginTransaction()) // This will fail if a transaction is already open
	{
	}

	~FAssetSearchDatabaseScopedTransaction()
	{
		Commit();
	}

	bool CommitOrRollback(const bool bShouldCommit)
	{
		if (bShouldCommit)
		{
			Commit();
			return true;
		}

		Rollback();
		return false;
	}

	void Commit()
	{
		if (bHasTransaction)
		{
			verify(Statements.CommitTransaction());
			bHasTransaction = false;
		}
	}

	void Rollback()
	{
		if (bHasTransaction)
		{
			verify(Statements.RollbackTransaction());
			bHasTransaction = false;
		}
	}

private:
	FAssetSearchDatabaseStatements& Statements;
	bool bHasTransaction;
};

FAssetSearchDatabase::FAssetSearchDatabase()
	: Database(MakeUnique<FSQLiteDatabase>())
	, DatabaseFileName(TEXT("AssetSearch.db"))
{
}

FAssetSearchDatabase::~FAssetSearchDatabase()
{
	Close();
}

bool FAssetSearchDatabase::IsValid() const
{
	return Database->IsValid();
}

bool FAssetSearchDatabase::Open(const FString& InSessionPath)
{
	return Open(InSessionPath, ESQLiteDatabaseOpenMode::ReadWriteCreate);
}

bool FAssetSearchDatabase::Open(const FString& InSessionPath, const ESQLiteDatabaseOpenMode InOpenMode)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FAssetSearchDatabase::Open");
	SessionPath = InSessionPath;

	if (Database->IsValid())
	{
		return false;
	}

	if (!Database->Open(*(InSessionPath / DatabaseFileName), ESQLiteDatabaseOpenMode::ReadWriteCreate))
	{
		UE_LOGF(LogAssetSearch, Error, "Failed to open database for '%ls': %ls", *InSessionPath, *GetLastError());
		return false;
	}

	if (bEnableIntegrityChecks && !Database->PerformQuickIntegrityCheck())
	{
		UE_LOGF(LogAssetSearch, Error, "Database failed integrity check, deleting.");

		const bool bDeleteTheDatabase = true;
		Close(bDeleteTheDatabase);

		return false;
	}

	// Set the database to use exclusive WAL mode for performance (exclusive works even on platforms without a mmap implementation)
	// Set the database "NORMAL" fsync mode to only perform a fsync when check-pointing the WAL to the main database file (fewer fsync calls are better for performance, with a very slight loss of WAL durability if the power fails)
	Database->Execute(TEXT("PRAGMA journal_mode=WAL;"));
	Database->Execute(TEXT("PRAGMA synchronous=FULL;"));
	Database->Execute(TEXT("PRAGMA cache_size=1000;"));
	Database->Execute(TEXT("PRAGMA page_size=65535;"));
	Database->Execute(TEXT("PRAGMA locking_mode=EXCLUSIVE;"));

	int32 LoadedDatabaseVersion = 0;
	Database->GetUserVersion(LoadedDatabaseVersion);
	if (LoadedDatabaseVersion != (int32)EAssetSearchDatabaseVersion::Empty)
	{
		if (LoadedDatabaseVersion > (int32)EAssetSearchDatabaseVersion::LatestVersion)
		{
			Close();
			UE_LOGF(LogAssetSearch, Error, "Failed to open database for '%ls': Database is too new (version %d, expected = %d)", *InSessionPath, LoadedDatabaseVersion, (int32)EAssetSearchDatabaseVersion::LatestVersion);
			return false;
		}
		else if (LoadedDatabaseVersion < (int32)EAssetSearchDatabaseVersion::LatestVersion)
		{
			Close(true);
			UE_LOGF(LogAssetSearch, Log, "Opened database '%ls': Database is too old (version %d, expected = %d), creating new database", *InSessionPath, LoadedDatabaseVersion, (int32)EAssetSearchDatabaseVersion::LatestVersion);
			return Open(InSessionPath, InOpenMode);
		}
	}

	// Create our required tables
	//========================================================================
	if (!ensure(Database->Execute(TEXT("CREATE TABLE IF NOT EXISTS table_assets(assetid INTEGER PRIMARY KEY, asset_name, asset_class, asset_path TEXT UNIQUE, index_hash);"))))
	{
		LogLastError();
		Close();
		return false;
	}

	if (!ensure(Database->Execute(TEXT("CREATE TABLE IF NOT EXISTS table_asset_properties(rowid INTEGER PRIMARY KEY, object_name, object_path, object_native_class, property_name, property_field, property_class, value_text, value_hidden, assetid INTEGER, FOREIGN KEY(assetid) REFERENCES table_assets(assetid));"))))
	{
		LogLastError();
		Close();
		return false;
	}

	if (!ensure(Database->Execute(TEXT("CREATE VIEW IF NOT EXISTS view_asset_properties AS SELECT rowid, object_name, object_path, object_native_class, property_name, property_field, property_class, value_text, value_hidden, table_asset_properties.assetid as assetid, table_assets.asset_name AS asset_name, table_assets.asset_class AS asset_class, table_assets.asset_path AS asset_path, table_assets.index_hash AS index_hash FROM table_asset_properties INNER JOIN table_assets on table_assets.assetid = table_asset_properties.assetid;"))))
	{
		LogLastError();
		Close();
		return false;
	}

	if (!ensure(Database->Execute(TEXT("CREATE VIRTUAL TABLE IF NOT EXISTS table_asset_properties_fts USING FTS5(asset_name UNINDEXED, asset_class UNINDEXED, asset_path UNINDEXED, object_name UNINDEXED, object_path UNINDEXED, object_native_class UNINDEXED, property_name UNINDEXED, property_field UNINDEXED, property_class UNINDEXED, value_text, value_hidden, assetid UNINDEXED, content=view_asset_properties, content_rowid=rowid);"))))
	{
		LogLastError();
		Close();
		return false;
	}

	if (!ensure(Database->Execute(TEXT(" DROP TRIGGER IF EXISTS table_assets_delete;"))))
	{
		LogLastError();
		Close();
		return false;
	}

	if (!ensure(Database->Execute(
		TEXT(" CREATE TRIGGER table_assets_delete BEFORE DELETE ON table_assets BEGIN")
		TEXT("     DELETE FROM table_asset_properties WHERE assetid == old.assetid;")
		TEXT(" END;")
	)))
	{
		LogLastError();
		Close();
		return false;
	}

	if (!ensure(Database->Execute(TEXT(" DROP TRIGGER IF EXISTS table_asset_properties_insert;"))))
	{
		LogLastError();
		Close();
		return false;
	}

	if (!ensure(Database->Execute(
		TEXT("CREATE TRIGGER table_asset_properties_insert AFTER INSERT ON table_asset_properties BEGIN")
		TEXT("	INSERT INTO table_asset_properties_fts(rowid, value_text, value_hidden) ")
		TEXT("		VALUES (new.rowid, new.value_text, new.value_hidden);")
		TEXT("END;")
		)))
	{
		LogLastError();
		Close();
	}

	if (!ensure(Database->Execute(
		TEXT(" DROP TRIGGER IF EXISTS table_asset_properties_delete;")
	)))
	{
		LogLastError();
		Close();
		return false;
	}

	if (!ensure(Database->Execute(
		TEXT("CREATE TRIGGER table_asset_properties_delete AFTER DELETE ON table_asset_properties BEGIN")
		TEXT("	INSERT INTO table_asset_properties_fts(table_asset_properties_fts, rowid, value_text, value_hidden) ")
		TEXT("		VALUES('delete', old.rowid, old.value_text, old.value_hidden);")
		TEXT("END;")
	)))
	{
		LogLastError();
		Close();
		return false;
	}

	if (!ensure(Database->Execute(
		TEXT(" DROP TRIGGER IF EXISTS table_asset_properties_update;")
	)))
	{
		LogLastError();
		Close();
		return false;
	}

	if (!ensure(Database->Execute(
		TEXT("CREATE TRIGGER table_asset_properties_update AFTER UPDATE ON table_asset_properties BEGIN")
		TEXT("	INSERT INTO table_asset_properties_fts(table_asset_properties_fts, rowid, value_text, value_hidden) ")
		TEXT("		VALUES('delete', old.rowid, old.value_text, old.value_hidden);")
		TEXT("	INSERT INTO table_asset_properties_fts(rowid, value_text, value_hidden) ")
		TEXT("		VALUES (new.rowid, new.value_text, new.value_hidden);")
		TEXT("END;")
	)))
	{
		LogLastError();
		Close();
		return false;
	}

	if (!ensure(Database->Execute(
		TEXT("CREATE UNIQUE INDEX IF NOT EXISTS asset_path_index ON table_assets(asset_path);")
	)))
	{
		LogLastError();
		Close();
		return false;
	}

	if (!ensure(Database->Execute(
		TEXT("CREATE INDEX IF NOT EXISTS assetid_index ON table_asset_properties(assetid);")
	)))
	{
		LogLastError();
		Close();
		return false;
	}

	//CREATE INDEX blueprint_nodes ON table_assets(json_extract(json, '$.Blueprint.Nodes'))

	// The database will have the latest schema at this point, so update the user-version
	if (!Database->SetUserVersion((int32)EAssetSearchDatabaseVersion::LatestVersion))
	{
		Close();
		return false;
	}

	// Create our required prepared statements
	Statements = MakeUnique<FAssetSearchDatabaseStatements>(*Database);
	if (!ensure(Statements->CreatePreparedStatements()))
	{
		Close();
		return false;
	}

	if (bEnableIntegrityChecks && !Database->PerformQuickIntegrityCheck())
	{
		UE_LOGF(LogAssetSearch, Error, "Database failed integrity check, deleting.");

		const bool bDeleteTheDatabase = true;
		Close(bDeleteTheDatabase);

		return false;
	}

	return true;
}

bool FAssetSearchDatabase::Close(const bool InDeleteDatabase)
{
	if (!Database->IsValid())
	{
		return false;
	}

	// Need to destroy prepared statements before the database can be closed
	Statements.Reset();

	if (!Database->Close())
	{
		UE_LOGF(LogAssetSearch, Error, "Failed to close database for '%ls': %ls", *SessionPath, *GetLastError());
		return false;
	}

	if (InDeleteDatabase)
	{
		IFileManager::Get().Delete(*(SessionPath / DatabaseFileName), false);
	}

	SessionPath.Reset();

	return true;
}

FString FAssetSearchDatabase::GetFilename() const
{
	return Database->GetFilename();
}

FString FAssetSearchDatabase::GetLastError() const
{
	return Database->GetLastError();
}

void FAssetSearchDatabase::LogLastError() const
{
	UE_LOGF(LogAssetSearch, Error, "Database '%ls' Error: %ls", *SessionPath, *GetLastError());
}

bool FAssetSearchDatabase::IsAssetUpToDate(const FAssetData& InAssetData, const FString& IndexedJsonHash)
{
	if (ensure(Statements))
	{
		return Statements->IsAssetUpToDate(InAssetData, IndexedJsonHash);
	}

	return false;
}

void FAssetSearchDatabase::AddOrUpdateAsset(const FAssetData& InAssetData, const FString& IndexedJson, const FString& IndexedJsonHash)
{
	if (ensure(Statements))
	{
		if (!ensure(Statements->DeleteEntriesForAsset(InAssetData)))
		{
			LogLastError();
		}

		if (!ensure(Statements->AddSearchRecord(InAssetData, IndexedJson, IndexedJsonHash)))
		{
			LogLastError();
		}
	}
}

bool FAssetSearchDatabase::EnumerateSearchResults(const FString& QueryText, TFunctionRef<bool(FSearchRecord&&)> InCallback)
{
	bool bSuccess = false;

	bSuccess |= Statements->SearchAssets(QueryText, [&InCallback](FSearchRecord&& InResult)
	{
		return InCallback(MoveTemp(InResult))
			? ESQLitePreparedStatementExecuteRowResult::Continue
			: ESQLitePreparedStatementExecuteRowResult::Stop;
	});

	bSuccess |= Statements->SearchAssetProperties(QueryText, [&InCallback](FSearchRecord&& InResult)
	{
		return InCallback(MoveTemp(InResult))
			? ESQLitePreparedStatementExecuteRowResult::Continue
			: ESQLitePreparedStatementExecuteRowResult::Stop;
	});

	return bSuccess;
}

int64 FAssetSearchDatabase::GetTotalSearchRecords() const
{
	if (ensure(Statements))
	{
		return Statements->GetTotalSearchRecords();
	}

	return INDEX_NONE;
}

void FAssetSearchDatabase::RemoveAsset(const FAssetData& InAssetData)
{
	if (!ensure(Statements->DeleteEntriesForAsset(InAssetData)))
	{
		LogLastError();
	}
}

void FAssetSearchDatabase::RemoveAssetsNotInThisSet(const TArray<FAssetData>& InAssets)
{
	TSet<FString> InAssetPaths;
	for (const FAssetData& InAsset : InAssets)
	{
		// If it's a redirector act like it has been removed from the system,
		// we don't want old duplicate entries for it.
		if (InAsset.IsRedirector())
		{
			continue;
		}

		InAssetPaths.Add(InAsset.GetObjectPathString());
	}

	TArray<FString> MissingAssets;

	Statements->GetAllAssets([&MissingAssets, &InAssetPaths](FString&& InResult)
	{
		if (!InAssetPaths.Contains(InResult))
		{
			MissingAssets.Emplace(InResult);
		}

		return ESQLitePreparedStatementExecuteRowResult::Continue;
	});

	for (const FString& MissingAsset : MissingAssets)
	{
		Statements->DeleteEntriesForAsset(MissingAsset);
	}
}

FString ConvertToFullTextSearchQuery(const FString& QueryText)
{
	TStringBuilder<512> Q;

	FTextFilterExpressionEvaluator Eval(ETextFilterExpressionEvaluatorMode::BasicString);
	const TArray<FExpressionToken>& Tokens = Eval.GetFilterExpressionTokens();
	if (Eval.SetFilterText(FText::FromString(QueryText)))
	{
		TArray<FString> TokenStreak;
		bool bBreakSteak = false;

		for (int32 i = 0; i < Tokens.Num(); i++)
		{
			const FExpressionToken& Token = Tokens[i];
			const FStringToken& TokenContext = Token.Context;
			const FString TokenString = TokenContext.GetString();
			
			TStringBuilder<64> Phrase;
			if (Token.Node.Cast<TextFilterExpressionParser::FTextToken>())
			{
				if (TokenString.StartsWith(TEXT("\"")) && TokenString.EndsWith(TEXT("\"")))
				{
					Phrase.Append(TokenString);
					Phrase.Append(TEXT(" "));
					bBreakSteak = true;
				}
				else
				{
					Phrase.Append(TEXT("\""));
					Phrase.Append(TokenString);
					Phrase.Append(TEXT("\" * "));

					TokenStreak.Add(TokenString);
				}
			}
			else if (Token.Node.Cast<TextFilterExpressionParser::FOr>())
			{
				Phrase.Append(TEXT(" OR "));
				bBreakSteak = true;
			}
			else if (Token.Node.Cast<TextFilterExpressionParser::FAnd>())
			{
				//Phrase.Append(TEXT(" AND "));
				//bBreakSteak = true;
			}
			else
			{
				bBreakSteak = true;
			}

			if (bBreakSteak)
			{
				if (TokenStreak.Num() > 1)
				{
					Q.Append(TEXT(" OR "));
					Q.Append(TEXT("\""));
					for (const FString& SimpleString : TokenStreak)
					{
						Q.Append(SimpleString);
					}
					Q.Append(TEXT("\""));
				}

				bBreakSteak = false;
				TokenStreak.Reset();
			}

			Q.Append(Phrase);
		}

		if (TokenStreak.Num() > 1)
		{
			Q.Append(TEXT(" OR "));
			Q.Append(TEXT("\""));
			for (const FString& SimpleString : TokenStreak)
			{
				Q.Append(SimpleString);
			}
			Q.Append(TEXT("\""));
		}
	}
	else
	{
		TArray<FString> Phrases;
		QueryText.ParseIntoArray(Phrases, TEXT(" "), 1);

		for (FString Phrase : Phrases)
		{
			Q.Append(TEXT("\""));
			Q.Append(Phrase);
			Q.Append(TEXT("\" * "));
		}

		Q.Append(TEXT(" OR "));
		Q.Append(TEXT("\""));
		Q.Append(QueryText.Replace(TEXT(" "), TEXT("")));
		Q.Append(TEXT("\""));
	}

	return Q.ToString();
}
