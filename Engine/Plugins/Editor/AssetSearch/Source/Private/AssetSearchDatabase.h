// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Templates/UniquePtr.h"

struct FSearchRecord;

class FSQLiteDatabase;
enum class ESQLiteDatabaseOpenMode : uint8;
class FAssetSearchDatabaseStatements;
struct FAssetData;

template <typename FuncType> class TFunctionRef;

DECLARE_LOG_CATEGORY_EXTERN(LogAssetSearch, Log, All);

class FAssetSearchDatabase
{
public:
	FAssetSearchDatabase();
	~FAssetSearchDatabase();

	FAssetSearchDatabase(const FAssetSearchDatabase&) = delete;
	FAssetSearchDatabase& operator=(const FAssetSearchDatabase&) = delete;

	FAssetSearchDatabase(FAssetSearchDatabase&&) = default;
	FAssetSearchDatabase& operator=(FAssetSearchDatabase&&) = default;

	/**
	 * Is this a valid database? (ie, has been successfully opened).
	 */
	bool IsValid() const;

	/**
	 * Open (or create) a database file.
	 *
	 * @param InSessionPath				The root path to store all the data for this session under.
	 *
	 * @return True if the database was opened, false otherwise.
	 */
	bool Open(const FString& InSessionPath);

	/**
	 * Open (or create) a database file.
	 *
	 * @param InSessionPath				The root path to store all the data for this session under.
	 * @param InOpenMode				How should the database be opened?
	 *
	 * @return True if the database was opened, false otherwise.
	 */
	bool Open(const FString& InSessionPath, const ESQLiteDatabaseOpenMode InOpenMode);

	/**
	 * Close an open database file.
	 *
	 * @param InDeleteDatabase			True if the session database and its associated data should be deleted after closing the database.
	 */
	bool Close(const bool InDeleteDatabase = false);

	/**
	 * Get the filename of the currently open database, or an empty string.
	 * @note The returned filename will be an absolute pathname.
	 */
	FString GetFilename() const;

	/**
	 * Get the last error reported by this database.
	 */
	FString GetLastError() const;

	/**
	 * Is the asset cache up to date for this asset?
	 */
	bool IsAssetUpToDate(const FAssetData& InAssetData, const FString& IndexedJsonHash);

	/**
	 * Adds an asset to the search database.
	 */
	void AddOrUpdateAsset(const FAssetData& InAssetData, const FString& IndexedJson, const FString& IndexedJsonHash);

	/**
	 * Search for some stuff.
	 */
	bool EnumerateSearchResults(const FString& QueryText, TFunctionRef<bool(FSearchRecord&&)> InCallback);

	/**
	 * Get the total number of records.
	 */
	int64 GetTotalSearchRecords() const;

	/**
	 * Remove asset from the the search database.
	 */
	void RemoveAsset(const FAssetData& InAssetData);

	/**
	 * Remove asset from the the search database that are not in this set of assets.
	 */
	void RemoveAssetsNotInThisSet(const TArray<FAssetData>& InAssets);
	
	bool bEnableIntegrityChecks = false;

private:
	void LogLastError() const;

private:
	/** Internal SQLite database */
	TUniquePtr<FSQLiteDatabase> Database;

	/** The database filename name we use */
	FString DatabaseFileName;

	/** Root path to store all session data under */
	FString SessionPath;

	/** Prepared statements for the currently open database */
	TUniquePtr<FAssetSearchDatabaseStatements> Statements;
};
