// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetSearchManager.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "DerivedDataCacheInterface.h"

#include "IAssetSearchModule.h"
#include "Misc/Paths.h"
#include "HAL/RunnableThread.h"
#include "Misc/App.h"
#include "StudioTelemetry.h"
#include "AnalyticsEventAttribute.h"
#include "Misc/PackageName.h"
#include "WidgetBlueprint.h"
#include "Engine/DataTable.h"
#include "Engine/DataAsset.h"
#include "SearchSerializer.h"
#include "Sound/DialogueWave.h"
#include "Settings/SearchProjectSettings.h"
#include "Settings/SearchUserSettings.h"
#include "Sound/SoundCue.h"
#include "Misc/ScopedSlowTask.h"
#include "Editor.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/ObjectSaveContext.h"
#include "FileHelpers.h"
#include "Misc/MessageDialog.h"
#include "Engine/CurveTable.h"
#include "Materials/Material.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialParameterCollection.h"
#include "Materials/MaterialInstance.h"
#include "Hash/CityHash.h"

#include "Indexers/GenericObjectIndexer.h"
#include "Indexers/DataTableIndexer.h"
#include "Indexers/BlueprintIndexer.h"
#include "Indexers/WidgetBlueprintIndexer.h"
#include "Indexers/CurveTableIndexer.h"
#include "Indexers/DialogueWaveIndexer.h"
#include "Indexers/LevelIndexer.h"
#include "Indexers/ActorIndexer.h"
#include "Indexers/SoundCueIndexer.h"
#include "Indexers/MaterialExpressionIndexer.h"
#include "Providers/AssetRegistrySearchProvider.h"


#define LOCTEXT_NAMESPACE "FAssetSearchManager"

static bool bForceEnableSearch = false;
FAutoConsoleVariableRef CVarDisableUniversalSearch(
	TEXT("Search.ForceEnable"),
	bForceEnableSearch,
	TEXT("Enable universal search")
);

static bool bTryIndexAssetsOnLoad = false;
FAutoConsoleVariableRef CVarTryIndexAssetsOnLoad(
	TEXT("Search.TryIndexAssetsOnLoad"),
	bTryIndexAssetsOnLoad,
	TEXT("Tries to index assets on load.")
);

static bool bTryToGCDuringMissingIndexing = false;
FAutoConsoleVariableRef CVarTryToGCDuringMissingIndexing(
	TEXT("Search.TryToGCDuringMissingIndexing"),
	bTryToGCDuringMissingIndexing,
	TEXT("Tries to GC occasionally while indexing missing items.")
);

//DEFINE_LOG_CATEGORY_STATIC(LogAssetSearch, Log, All);

class FUnloadPackageScope
{
public:
	FUnloadPackageScope()
	{
		if (bTryToGCDuringMissingIndexing)
		{
			FCoreUObjectDelegates::OnAssetLoaded.AddRaw(this, &FUnloadPackageScope::OnAssetLoaded);
		}
	}

	~FUnloadPackageScope()
	{
		if (bTryToGCDuringMissingIndexing)
		{
			FCoreUObjectDelegates::OnAssetLoaded.RemoveAll(this);
			TryUnload(true);
		}
	}

	int32 TryUnload(bool bResetTrackedObjects)
	{
		if (!bTryToGCDuringMissingIndexing)
		{
			return 0;
		}

		TArray<TWeakObjectPtr<UObject>> PackageObjectPtrs;

		for (const FObjectKey& LoadedObjectKey : ObjectsLoaded)
		{
			if (UObject* LoadedObject = LoadedObjectKey.ResolveObjectPtr())
			{
				UPackage* Package = LoadedObject->GetOutermost();

				TArray<UObject*> PackageObjects;
				GetObjectsWithOuter(Package, PackageObjects, EGetObjectsFlags::None);

				for (UObject* PackageObject : PackageObjects)
				{
					PackageObject->ClearFlags(RF_Standalone);
					PackageObjectPtrs.Add(PackageObject);
				}
			}
		}

		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
		
		int32 NumRemoved = 0;
		for (int32 LoadedAssetIndex = 0; LoadedAssetIndex < PackageObjectPtrs.Num(); LoadedAssetIndex++)
		{
			TWeakObjectPtr<UObject> PackageObjectPtr = PackageObjectPtrs[LoadedAssetIndex];

			if (UObject* LoadedObject = PackageObjectPtr.Get())
			{
				//FReferencerInformationList ReferencesIncludingUndo;
				//bool bReferencedInMemoryOrUndoStack = IsReferenced(LoadedObject, GARBAGE_COLLECTION_KEEPFLAGS, EInternalObjectFlags_GarbageCollectionKeepFlags, true, &ReferencesIncludingUndo);

				LoadedObject->SetFlags(RF_Standalone);
			}
			else
			{
				NumRemoved++;
			}
		}

		if (bResetTrackedObjects)
		{
			ObjectsLoaded.Reset();
		}
		else
		{
			for (int32 ObjectIndex = 0; ObjectIndex < ObjectsLoaded.Num(); ObjectIndex++)
			{
				if (!ObjectsLoaded[ObjectIndex].ResolveObjectPtr())
				{
					ObjectsLoaded.RemoveAt(ObjectIndex);
					ObjectIndex--;
				}
			}
		}

		return NumRemoved;
	}

	int32 GetObjectsLoaded() const
	{
		return ObjectsLoaded.Num();
	}

private:
	void OnAssetLoaded(UObject* InObject)
	{
		ObjectsLoaded.Add(FObjectKey(InObject));
	}

private:
	TArray<FObjectKey> ObjectsLoaded;
	TArray<UClass*> ClassFilters;
};

const FName FAssetSearchManager::AssetSearchIndexVersionTag(TEXT("ASI_Version"));
const FName FAssetSearchManager::AssetSearchIndexHashTag(TEXT("ASI_Hash"));
const FName FAssetSearchManager::AssetSearchIndexDataTag(TEXT("ASI_Data"));

FAssetSearchManager::FAssetSearchManager()
{
	PendingDatabaseUpdates = 0;
	IsAssetUpToDateCount = 0;
	ActiveDownloads = 0;
	DownloadQueueCount = 0;
	TotalSearchRecords = 0;
	LastRecordCountUpdateSeconds = 0;

	IntermediateStorage = GetDefault<USearchProjectSettings>()->IntermediateStorage;

	RunThread = false;
}

FAssetSearchManager::~FAssetSearchManager()
{
	RunThread = false;

	if (DatabaseThread)
	{
		DatabaseThread->WaitForCompletion();
	}

	StopScanningAssets();

	UPackage::PackageSavedWithContextEvent.RemoveAll(this);
	FCoreUObjectDelegates::OnAssetLoaded.RemoveAll(this);
	UObject::FAssetRegistryTag::OnGetExtraObjectTagsWithContext.RemoveAll(this);

	FTSTicker::RemoveTicker(TickerHandle);
}

void FAssetSearchManager::Start()
{
	RegisterAssetIndexer(UDataAsset::StaticClass(), MakeUnique<FGenericObjectIndexer>("DataAsset"));
	RegisterAssetIndexer(UDataTable::StaticClass(), MakeUnique<FDataTableIndexer>());
	RegisterAssetIndexer(UCurveTable::StaticClass(), MakeUnique<FCurveTableIndexer>());
	RegisterAssetIndexer(UBlueprint::StaticClass(), MakeUnique<FBlueprintIndexer>());
	RegisterAssetIndexer(UWidgetBlueprint::StaticClass(), MakeUnique<FWidgetBlueprintIndexer>());
	RegisterAssetIndexer(UDialogueWave::StaticClass(), MakeUnique<FDialogueWaveIndexer>());
	RegisterAssetIndexer(UWorld::StaticClass(), MakeUnique<FLevelIndexer>());
	RegisterAssetIndexer(AActor::StaticClass(), MakeUnique<FActorIndexer>());
	RegisterAssetIndexer(USoundCue::StaticClass(), MakeUnique<FSoundCueIndexer>());
	RegisterAssetIndexer(UMaterial::StaticClass(), MakeUnique<FMaterialExpressionIndexer>("Material"));
	RegisterAssetIndexer(UMaterialFunction::StaticClass(), MakeUnique<FMaterialExpressionIndexer>("MaterialFunction"));
	RegisterAssetIndexer(UMaterialParameterCollection::StaticClass(), MakeUnique<FGenericObjectIndexer>("MaterialParameterCollection"));
	RegisterAssetIndexer(UMaterialInstance::StaticClass(), MakeUnique<FGenericObjectIndexer>("MaterialInstance"));

	RegisterSearchProvider(TEXT("AssetRegistry"), MakeUnique<FAssetRegistrySearchProvider>());

	if (IntermediateStorage == ESearchIntermediateStorage::DerivedDataCache)
	{
		UPackage::PackageSavedWithContextEvent.AddRaw(this, &FAssetSearchManager::HandlePackageSaved);
	}

	FCoreUObjectDelegates::OnAssetLoaded.AddRaw(this, &FAssetSearchManager::OnAssetLoaded);

	TickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FAssetSearchManager::Tick_GameThread), 0);

	SearchDatabase.bEnableIntegrityChecks = GetDefault<USearchUserSettings>()->bEnableIntegrityChecks;
	FileInfoDatabase.bEnableIntegrityChecks = GetDefault<USearchUserSettings>()->bEnableIntegrityChecks;
	RunThread = true;
	DatabaseThread = FRunnableThread::Create(this, TEXT("UniversalSearch"), 0, TPri_BelowNormal);

	if (IntermediateStorage == ESearchIntermediateStorage::AssetTagData)
	{
		UObject::FAssetRegistryTag::OnGetExtraObjectTagsWithContext.AddRaw(this, &FAssetSearchManager::HandleOnGetExtraObjectTags);
	}
}

// Called from within AssetSearchManagerCS lock
void FAssetSearchManager::UpdateScanningAssets()
{
	bool bTargetState = GetDefault<USearchUserSettings>()->bEnableSearch;

	if (GIsBuildMachine || FApp::IsUnattended())
	{
		bTargetState = false;
	}

	if (bForceEnableSearch)
	{
		bTargetState = true;
	}

	if (bTargetState != bStarted)
	{
		bStarted = bTargetState;

		if (bTargetState)
		{
			StartScanningAssets();
		}
		else
		{
			StopScanningAssets();
		}
	}
}

// Called from within AssetSearchManagerCS lock
void FAssetSearchManager::StartScanningAssets()
{
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	if (AssetRegistry.IsLoadingAssets())
	{
		AssetRegistry.OnFilesLoaded().AddRaw(this, &FAssetSearchManager::StartScanningAssetsOnAssetScanFinished);
	}
	else if (!bFinishedOnce)
	{
		StartScanningAssetsOnAssetScanFinishedInternal();
	}
	else
	{
		AssetRegistry.OnAssetAdded().AddRaw(this, &FAssetSearchManager::OnAssetAdded);
		AssetRegistry.OnAssetRemoved().AddRaw(this, &FAssetSearchManager::OnAssetRemoved);
	}
}

void FAssetSearchManager::StopScanningAssets()
{
	if (FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>("AssetRegistry"))
	{
		IAssetRegistry* AssetRegistry = AssetRegistryModule->TryGet();
		if (AssetRegistry)
		{
			AssetRegistry->OnAssetAdded().RemoveAll(this);
			AssetRegistry->OnAssetRemoved().RemoveAll(this);
			AssetRegistry->OnFilesLoaded().RemoveAll(this);
		}
	}

	ProcessAssetQueue.Reset();
	AssetNeedingReindexing.Reset();
}

void FAssetSearchManager::TryConnectToDatabase()
{
	if (!bDatabaseOpen)
	{
		check(!IsInGameThread());

		if ((FPlatformTime::Seconds() - LastConnectionAttempt) > 30)
		{
			LastConnectionAttempt = FPlatformTime::Seconds();

			const FString SessionPath = FPaths::ConvertRelativePathToFull(FPaths::Combine(FPaths::ProjectSavedDir(), TEXT("Search")));
			
			if (!FileInfoDatabase.IsValid())
			{
				FScopeLock ScopedLock(&FileInfoDatabaseCS);

				if (!FileInfoDatabase.Open(SessionPath))
				{
					return;
				}
			}

			if (!SearchDatabase.IsValid())
			{
				FScopeLock ScopedLock(&SearchDatabaseCS);

				if (!SearchDatabase.Open(SessionPath))
				{
					return;
				}
			}

			bDatabaseOpen = true;
		}
	}
}

FSearchStats FAssetSearchManager::GetStats() const
{
	FSearchStats Stats;
	Stats.Scanning = ProcessAssetQueue.Num();
	Stats.Processing = IsAssetUpToDateCount + DownloadQueueCount + ActiveDownloads;
	Stats.Updating = PendingDatabaseUpdates;
	Stats.TotalRecords = TotalSearchRecords;
	Stats.AssetsMissingIndex = AssetNeedingReindexing.Num();

	return Stats;
}

void FAssetSearchManager::RegisterAssetIndexer(const UClass* AssetClass, TUniquePtr<IAssetIndexer>&& Indexer)
{
	FScopeLock Lock(&AssetSearchManagerCS);

	Indexers.Add(AssetClass->GetFName(), MoveTemp(Indexer));
}

void FAssetSearchManager::RegisterSearchProvider(FName SearchProviderName, TUniquePtr<ISearchProvider>&& InSearchProvider)
{
	FScopeLock Lock(&AssetSearchManagerCS);

	SearchProviders.Add(SearchProviderName, MoveTemp(InSearchProvider));
}

void FAssetSearchManager::OnAssetAdded(const FAssetData& InAssetData)
{
	FScopeLock Lock(&AssetSearchManagerCS);

	static const FString EngineContentPathWithSlash = FPackageName::FilenameToLongPackageName(FPaths::EngineContentDir());
	static const FString DeveloperPathWithSlash = FPackageName::FilenameToLongPackageName(FPaths::GameDevelopersDir());
	static const FString UsersDeveloperPathWithSlash = FPackageName::FilenameToLongPackageName(FPaths::GameUserDeveloperDir());

	const FString PackageName = InAssetData.PackageName.ToString();
	const FString PackageMountPoint = FPackageName::GetPackageMountPoint(PackageName).ToString();
	FString PackageFilename;
	FPackageName::TryConvertLongPackageNameToFilename(PackageName, PackageFilename);

	// Don't index things in the engine directory if we're storing data in asset tags, since we don't want to mutate the engine data.
	if (IntermediateStorage == ESearchIntermediateStorage::AssetTagData)
	{
		if (PackageName.StartsWith(EngineContentPathWithSlash) || 
		    FPaths::IsUnderDirectory(PackageFilename, FPaths::EnginePluginsDir()))
		{
			return;
		}
	}
	
	// Don't process stuff in the other developer folders.
	if (PackageName.StartsWith(DeveloperPathWithSlash))
	{
		if (!PackageName.StartsWith(UsersDeveloperPathWithSlash))
		{
			return;
		}
	}

	// Ignore it if the package is in one of the project ignored paths.
	const USearchProjectSettings* ProjectSettings = GetDefault<USearchProjectSettings>();
	for (const FDirectoryPath& IgnoredPath : ProjectSettings->IgnoredPaths)
	{
		if (PackageName.StartsWith(IgnoredPath.Path))
		{
			return;
		}
	}

	// Ignore it if the package is in one of the user ignored paths.
	const USearchUserSettings* UserSettings = GetDefault<USearchUserSettings>();
	for (const FDirectoryPath& IgnoredPath : UserSettings->IgnoredPaths)
	{
		if (PackageName.StartsWith(IgnoredPath.Path))
		{
			return;
		}
	}

	// Don't index redirectors, just act like they don't exist.
	if (InAssetData.IsRedirector())
	{
		return;
	}

	FAssetOperation Operation;
	Operation.Asset = InAssetData;
	ProcessAssetQueue.Add(Operation);
}

void FAssetSearchManager::OnAssetRemoved(const FAssetData& InAssetData)
{
	FScopeLock Lock(&AssetSearchManagerCS);

	FAssetOperation Operation;
	Operation.Asset = InAssetData;
	Operation.bRemoval = true;
	ProcessAssetQueue.Add(Operation);
}

void FAssetSearchManager::StartScanningAssetsOnAssetScanFinished()
{
	FScopeLock Lock(&AssetSearchManagerCS);

	TArray<FAssetData> AllAssets = StartScanningAssetsOnAssetScanFinishedInternal();
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	AssetRegistry.OnFilesLoaded().RemoveAll(this);
	
	PendingDatabaseUpdates++;
	UpdateOperations.Enqueue([this, AssetsAvailable = MoveTemp(AllAssets)]() mutable {
		FScopeLock ScopedLock(&SearchDatabaseCS);
		//UE_LOGF(LogAssetSearch, Log, "");
		SearchDatabase.RemoveAssetsNotInThisSet(AssetsAvailable);
		PendingDatabaseUpdates--;
	});
}

// Called from within AssetSearchManagerCS lock
TArray<FAssetData> FAssetSearchManager::StartScanningAssetsOnAssetScanFinishedInternal()
{
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	AssetRegistry.OnAssetAdded().AddRaw(this, &FAssetSearchManager::OnAssetAdded);
	AssetRegistry.OnAssetRemoved().AddRaw(this, &FAssetSearchManager::OnAssetRemoved);

	TArray<FAssetData> AllAssets;
	AssetRegistry.GetAllAssets(AllAssets, true);

	for (const FAssetData& Data : AllAssets)
	{
		OnAssetAdded(Data);
	}

	bFinishedOnce = true;
	return AllAssets;
}

uint64 FAssetSearchManager::GetTextHash(FStringView PackageRelativeExportPath) const
{
	if (PackageRelativeExportPath.Len() == 0)
	{
		return 0;
	}

	return CityHash64(reinterpret_cast<const char*>(PackageRelativeExportPath.GetData() + 1), (PackageRelativeExportPath.Len() - 1) * sizeof(TCHAR));
}

void FAssetSearchManager::HandleOnGetExtraObjectTags(FAssetRegistryTagsContext Context)
{
	if (!Context.IsFullUpdate())
	{
		return;
	}

	const UObject* Object = Context.GetObject();
	const ITargetPlatform* TargetPlatform = Context.GetTargetPlatform();
	if (GIsEditor && !Object->GetOutermost()->HasAnyPackageFlags(PKG_ForDiffing) && !IsRunningCookCommandlet())
	{
		if (!GEditor->IsAutosaving())
		{
			if (!TargetPlatform || TargetPlatform->HasEditorOnlyData())
			{
				FScopeLock Lock(&AssetSearchManagerCS);

				FAssetData InAssetData(Object, FAssetData::ECreationFlags::SkipAssetRegistryTagsGathering);

				FString IndexedJson;
				const bool bWasIndexed = IndexAsset(InAssetData, Object, IndexedJson);
				if (bWasIndexed && !IndexedJson.IsEmpty())
				{
					const FString IndexVersion = GetBaseIndexKey(Object->GetClass());
					const FString IndexHash = LexToString(GetTextHash(IndexedJson));

					Context.AddTag(UObject::FAssetRegistryTag(FAssetSearchManager::AssetSearchIndexVersionTag, IndexVersion, UObject::FAssetRegistryTag::TT_Hidden));
					Context.AddTag(UObject::FAssetRegistryTag(FAssetSearchManager::AssetSearchIndexHashTag, IndexHash, UObject::FAssetRegistryTag::TT_Hidden));
					Context.AddTag(UObject::FAssetRegistryTag(FAssetSearchManager::AssetSearchIndexDataTag, IndexedJson, UObject::FAssetRegistryTag::TT_Hidden));

					if (!IndexedJson.IsEmpty())
					{
						AddOrUpdateAsset(InAssetData, IndexedJson, IndexHash);
					}
				}
			}
		}
	}
}

void FAssetSearchManager::HandlePackageSaved(const FString& PackageFilename, UPackage* Package, FObjectPostSaveContext ObjectSaveContext)
{
	check(IntermediateStorage == ESearchIntermediateStorage::DerivedDataCache);

	// Only execute if this is a user save
	if (ObjectSaveContext.IsProceduralSave())
	{
		return;
	}

	if (GIsEditor && !IsRunningCommandlet())
	{
		FScopeLock Lock(&AssetSearchManagerCS);
		TArray<UObject*> Objects;
		GetObjectsWithPackage(Package, Objects, EGetObjectsFlags::None);
		for (UObject* Entry : Objects)
		{
			RequestIndexAsset_DDC(Entry);
		}
	}
}

void FAssetSearchManager::OnAssetLoaded(UObject* InObject)
{
	if (bTryIndexAssetsOnLoad)
	{
		switch (IntermediateStorage)
		{
			case ESearchIntermediateStorage::DerivedDataCache:
				RequestIndexAsset_DDC(InObject);
				break;
			case ESearchIntermediateStorage::AssetTagData:
				// Don't need to do this on load, I mean, it's a little more accurate but if we're storing it
				// in tag data, it's almost certainly up to date unless someone changed the serialization.
				break;
		}
	}
}

bool FAssetSearchManager::RequestIndexAsset_DDC(const UObject* InAsset)
{
	FScopeLock Lock(&AssetSearchManagerCS);
	check(IntermediateStorage == ESearchIntermediateStorage::DerivedDataCache);

	if (GEditor == nullptr || GEditor->IsAutosaving())
	{
		return false;
	}

	if (IsAssetIndexable(InAsset))
	{
		TWeakObjectPtr<const UObject> AssetWeakPtr = InAsset;
		FAssetData AssetData(InAsset);

		return AsyncGetDerivedDataKey(AssetData, [this, AssetData, AssetWeakPtr](bool bSuccess, FString InDDCKey) {
			if (!bSuccess)
			{
				return;
			}

			UpdateOperations.Enqueue([this, AssetData, AssetWeakPtr, InDDCKey]() {
				FScopeLock ScopedLock(&SearchDatabaseCS);
				if (!SearchDatabase.IsAssetUpToDate(AssetData, InDDCKey))
				{
					AsyncMainThreadTask([this, AssetWeakPtr]() {
						StoreIndexForAsset_DDC(AssetWeakPtr.Get());
					});
				}
			});
		});
	}

	return false;
}

bool FAssetSearchManager::IsAssetIndexable(const UObject* InAsset) const
{
	if (InAsset && InAsset->IsAsset())
	{
		// If it's not a permanent package, and one we just loaded for diffing, don't index it.
		UPackage* Package = InAsset->GetOutermost();
		if (Package->HasAnyPackageFlags(/*LOAD_ForDiff | */LOAD_PackageForPIE | LOAD_ForFileDiff))
		{
			return false;
		}

		if (InAsset->HasAnyFlags(RF_Transient))
		{
			return false;
		}

		return true;
	}

	return false;
}

bool FAssetSearchManager::TryLoadIndexForAsset(const FAssetData& InAssetData)
{
	switch (IntermediateStorage)
	{
		case ESearchIntermediateStorage::DerivedDataCache:
			return TryLoadIndexForAsset_DDC(InAssetData);
		case ESearchIntermediateStorage::AssetTagData:
			return TryLoadIndexForAsset_Tags(InAssetData);
		default:
			ensure(false);
			return false;
	}
}

bool FAssetSearchManager::TryLoadIndexForAsset_Tags(const FAssetData& InAssetData)
{
	FScopeLock Lock(&AssetSearchManagerCS);
	check(IntermediateStorage == ESearchIntermediateStorage::AssetTagData);

	IsAssetUpToDateCount++;

	FString CurrentIndexerNameAndVersion = GetBaseIndexKey(InAssetData.GetClass());
	if (!CurrentIndexerNameAndVersion.IsEmpty())
	{
		const FString SavedIndexerNameAndVersion = InAssetData.GetTagValueRef<FString>(FAssetSearchManager::AssetSearchIndexVersionTag);
		const FString SavedIndexHash = InAssetData.GetTagValueRef<FString>(FAssetSearchManager::AssetSearchIndexHashTag);

		if (!SavedIndexerNameAndVersion.IsEmpty())
		{
			if (SavedIndexerNameAndVersion != CurrentIndexerNameAndVersion)
			{
				// Log need to update/missing...etc.
				AssetNeedingReindexing.Add(InAssetData);
			}

			UpdateOperations.Enqueue([this, InAssetData, SavedIndexHash]() {
				FScopeLock ScopedLock(&SearchDatabaseCS);
				if (!SearchDatabase.IsAssetUpToDate(InAssetData, SavedIndexHash))
				{
					AsyncMainThreadTask([this, InAssetData, SavedIndexHash]() {
						const FString IndexedJson = InAssetData.GetTagValueRef<FString>(FAssetSearchManager::AssetSearchIndexDataTag);

						IsAssetUpToDateCount--;

						if (!IndexedJson.IsEmpty())
						{
							AddOrUpdateAsset(InAssetData, IndexedJson, SavedIndexHash);
						}
					});
				}
				else
				{
					IsAssetUpToDateCount--;
				}
			});

			return true;
		}
		else
		{
 			AssetNeedingReindexing.Add(InAssetData);
		}
	}

	IsAssetUpToDateCount--;
	return false;
}

bool FAssetSearchManager::TryLoadIndexForAsset_DDC(const FAssetData& InAssetData)
{
	FScopeLock Lock(&AssetSearchManagerCS);
	check(IntermediateStorage == ESearchIntermediateStorage::DerivedDataCache);
	
	bool bAllowFetch = !GetDefault<USearchProjectSettings>()->bDisableDDC;
	if (!bAllowFetch)
	{
		return false;
	}

	const bool bSuccess = AsyncGetDerivedDataKey(InAssetData, [this, InAssetData](bool bSuccess, FString InDDCKey) {
		if (!bSuccess)
		{
			IsAssetUpToDateCount--;
			return;
		}
		
		FeedOperations.Enqueue([this, InAssetData, InDDCKey]() {
			FScopeLock ScopedLock(&SearchDatabaseCS);
			if (!SearchDatabase.IsAssetUpToDate(InAssetData, InDDCKey))
			{
				AsyncRequestDownload(InAssetData, InDDCKey);
			}

			IsAssetUpToDateCount--;
		});
	});

	if (bSuccess)
	{
		IsAssetUpToDateCount++;
	}

	return bSuccess;
}

void FAssetSearchManager::AsyncRequestDownload(const FAssetData& InAssetData, const FString& InDDCKey)
{
	FScopeLock Lock(&AssetSearchManagerCS);
	DownloadQueueCount++;

	FAssetDDCRequest DDCRequest;
	DDCRequest.AssetData = InAssetData;
	DDCRequest.DDCKey = InDDCKey;
	DownloadQueue.Enqueue(DDCRequest);
}

bool FAssetSearchManager::AsyncGetDerivedDataKey(const FAssetData& InAssetData, TFunction<void(bool, FString)> DDCKeyCallback)
{
	FScopeLock Lock(&AssetSearchManagerCS);
	check(IntermediateStorage == ESearchIntermediateStorage::DerivedDataCache);

	const FString AssetPath = InAssetData.PackagePath.ToString();
	if (AssetPath.Contains(FPackagePath::GetExternalActorsFolderName()) || AssetPath.Contains(FPackagePath::GetExternalObjectsFolderName()))
	{
		return false;
	}

	FString IndexersNamesAndVersions = GetIndexerVersion(InAssetData.GetClass());

	// If the indexer names and versions is empty, then we know it's not possible to index this type of thing.
	if (IndexersNamesAndVersions.IsEmpty())
	{
		return false;
	}

	UpdateOperations.Enqueue([this, InAssetData, IndexersNamesAndVersions, DDCKeyCallback]() {
		FAssetFileInfo FileInfo;

		{
			FScopeLock ScopedLock(&FileInfoDatabaseCS);
			FileInfoDatabase.AddOrUpdateFileInfo(InAssetData, FileInfo);
		}

		if (FileInfo.Hash.IsValid())
		{
			// The universal key for content is:
			// AssetSearch_V{SerializerVersion}_{IndexersNamesAndVersions}_{ObjectPathHash}_{FileOnDiskHash}

			const FString ObjectPathString = InAssetData.GetObjectPathString();

			FSHAHash ObjectPathHash;
			FSHA1::HashBuffer(*ObjectPathString, ObjectPathString.Len() * sizeof(FString::ElementType), ObjectPathHash.Hash);

			TStringBuilder<512> DDCKey;
			DDCKey.Append(TEXT("AssetSearch_V"));
			DDCKey.Append(LexToString(FSearchSerializer::GetVersion()));
			DDCKey.Append(TEXT("_"));
			DDCKey.Append(IndexersNamesAndVersions);
			DDCKey.Append(TEXT("_"));
			DDCKey.Append(ObjectPathHash.ToString());
			DDCKey.Append(TEXT("_"));
			DDCKey.Append(LexToString(FileInfo.Hash));

			const FString DDCKeyString = DDCKey.ToString();

			DDCKeyCallback(true, DDCKeyString);
		}
		else
		{
			UE_LOGF(LogAssetSearch, Warning, "%ls unable to hash file.", *InAssetData.PackageName.ToString());
			DDCKeyCallback(false, TEXT(""));
		}
	});

	return true;
}

bool FAssetSearchManager::HasIndexerForClass(const UClass* InAssetClass) const
{
	const UClass* IndexableClass = InAssetClass;
	while (IndexableClass)
	{
		if (Indexers.Contains(IndexableClass->GetFName()))
		{
			return true;
		}

		IndexableClass = IndexableClass->GetSuperClass();
	}

	return false;
}

FString FAssetSearchManager::GetBaseIndexKey(const UClass* InAssetClass) const
{
	TStringBuilder<256> VersionString;

	const FString AssetIndexerVersions = GetIndexerVersion(InAssetClass);
	if (!AssetIndexerVersions.IsEmpty())
	{
		VersionString.Append(TEXT("AssetSearch_V"));
		VersionString.Append(LexToString(FSearchSerializer::GetVersion()));
		VersionString.Append(TEXT("_"));
		VersionString.Append(AssetIndexerVersions);
	}

	return VersionString.ToString();
}

FString FAssetSearchManager::GetIndexerVersion(const UClass* InAssetClass) const
{
	TStringBuilder<256> VersionString;

	TArray<UClass*> NestedIndexedTypes;

	const UClass* IndexableClass = InAssetClass;
	while (IndexableClass)
	{
		if (const TUniquePtr<IAssetIndexer>* IndexerPtr = Indexers.Find(IndexableClass->GetFName()))
		{
			IAssetIndexer* Indexer = IndexerPtr->Get();
			VersionString.Append(Indexer->GetName());
			VersionString.Append(TEXT("_"));
			VersionString.Append(LexToString(Indexer->GetVersion()));

			Indexer->GetNestedAssetTypes(NestedIndexedTypes);
		}

		IndexableClass = IndexableClass->GetSuperClass();
	}

	for (UClass* NestedIndexedType : NestedIndexedTypes)
	{
		VersionString.Append(GetIndexerVersion(NestedIndexedType));
	}

	return VersionString.ToString();
}

bool FAssetSearchManager::IndexAsset(const FAssetData& InAssetData, const UObject* InAsset, FString& OutIndexedJson) const
{
	FScopeLock Lock(&AssetSearchManagerCS);

	if (IsAssetIndexable(InAsset) && HasIndexerForClass(InAsset->GetClass()))
	{
		FSearchSerializer Serializer(InAssetData, &OutIndexedJson);
		return Serializer.IndexAsset(InAsset, Indexers);
	}

	return false;
}

void FAssetSearchManager::StoreIndexForAsset(const UObject* InAsset)
{
	switch (IntermediateStorage)
	{
	case ESearchIntermediateStorage::DerivedDataCache:
		return StoreIndexForAsset_DDC(InAsset);
	case ESearchIntermediateStorage::AssetTagData:
		return StoreIndexForAsset_Tags(InAsset);
	default:
		ensure(false);
	}
}

void FAssetSearchManager::StoreIndexForAsset_Tags(const UObject* InAsset)
{
	FScopeLock Lock(&AssetSearchManagerCS);
	check(IntermediateStorage == ESearchIntermediateStorage::AssetTagData);

	// StoreIndexForAsset doesn't really do much of anything when it's stored in tag data, all we can really
	// do is mark the package dirty so that it can be saved.  There's really nothing else that need be done
	// to upgrade it, since the index data was already grabbed from the tag data upon discovery.
	InAsset->MarkPackageDirty();
}

void FAssetSearchManager::StoreIndexForAsset_DDC(const UObject* InAsset)
{
	FScopeLock Lock(&AssetSearchManagerCS);
	check(IntermediateStorage == ESearchIntermediateStorage::DerivedDataCache);

	FString IndexedJson;
	FAssetData InAssetData(InAsset, FAssetData::ECreationFlags::SkipAssetRegistryTagsGathering);
	const bool bWasIndexed = IndexAsset(InAssetData, InAsset, IndexedJson);

	if (bWasIndexed && !IndexedJson.IsEmpty())
	{
		AsyncGetDerivedDataKey(InAssetData, [this, InAssetData, IndexedJson](bool bSuccess, FString InDDCKey) {
			if (!bSuccess)
			{
				return;
			}

			AsyncMainThreadTask([this, InAssetData, IndexedJson, InDDCKey]() {
				check(IsInGameThread());

				bool bAllowPut = !GetDefault<USearchProjectSettings>()->bDisableDDC;
				if (bAllowPut)
				{
					FTCHARToUTF8 IndexedJsonUTF8(*IndexedJson);
					TArrayView<const uint8> IndexedJsonUTF8View((const uint8*)IndexedJsonUTF8.Get(), IndexedJsonUTF8.Length() * sizeof(UTF8CHAR));
					GetDerivedDataCacheRef().Put(*InDDCKey, IndexedJsonUTF8View, InAssetData.GetObjectPathString(), false);
				}

				AddOrUpdateAsset(InAssetData, IndexedJson, InDDCKey);
			});
		});
	}
}

void FAssetSearchManager::LoadDDCContentIntoDatabase(const FAssetData& InAsset, const TArray<uint8>& Content, const FString& DerivedDataKey)
{
	FUTF8ToTCHAR WByteBuffer((const ANSICHAR*)Content.GetData(), Content.Num());
	FString IndexedJson = FString::ConstructFromPtrSize(WByteBuffer.Get(), WByteBuffer.Length());

	AddOrUpdateAsset(InAsset, IndexedJson, DerivedDataKey);
}

void FAssetSearchManager::AddOrUpdateAsset(const FAssetData& InAssetData, const FString& IndexedJson, const FString& DerivedDataKey)
{
	FScopeLock Lock(&AssetSearchManagerCS);

	PendingDatabaseUpdates++;
	UpdateOperations.Enqueue([this, InAssetData, IndexedJson, DerivedDataKey]() {
		FScopeLock ScopedLock(&SearchDatabaseCS);
		SearchDatabase.AddOrUpdateAsset(InAssetData, IndexedJson, DerivedDataKey);
		PendingDatabaseUpdates--;
	});
}

bool FAssetSearchManager::Tick_GameThread(float DeltaTime)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FAssetSearchManager_Tick);

	FScopeLock Lock(&AssetSearchManagerCS);
	check(IsInGameThread());

	UpdateScanningAssets();

	ProcessGameThreadTasks();

	const USearchUserSettings* UserSettings = GetDefault<USearchUserSettings>();
	const FSearchPerformance& PerformanceLimits = bForceEnableSearch ? UserSettings->DefaultOptions : UserSettings->GetPerformanceOptions();

	int32 ScanLimit = PerformanceLimits.AssetScanRate;
	while (ProcessAssetQueue.Num() > 0 && ScanLimit > 0)
	{
		FAssetOperation Operation = ProcessAssetQueue.Pop(EAllowShrinking::No);
		FAssetData Asset = Operation.Asset;

		if (Operation.bRemoval)
		{
			PendingDatabaseUpdates++;
			UpdateOperations.Enqueue([this, Asset]() {
				FScopeLock ScopedLock(&SearchDatabaseCS);
				SearchDatabase.RemoveAsset(Asset);
				PendingDatabaseUpdates--;
			});
		}
		else
		{
			TryLoadIndexForAsset(Asset);
		}

		ScanLimit--;
	}

	while (!DownloadQueue.IsEmpty() && ActiveDownloads < PerformanceLimits.ParallelDownloads)
	{
		FAssetDDCRequest DDCRequest;
		bool bSuccess = DownloadQueue.Dequeue(DDCRequest);
		check(bSuccess);

		DownloadQueueCount--;
		ActiveDownloads++;

		DDCRequest.DDCHandle = GetDerivedDataCacheRef().GetAsynchronous(*DDCRequest.DDCKey, DDCRequest.AssetData.GetObjectPathString());
		ProcessDDCQueue.Enqueue(DDCRequest);
	}

	int32 MaxQueueProcesses = 1000;
	int32 DownloadProcessLimit = PerformanceLimits.DownloadProcessRate;
	while (!ProcessDDCQueue.IsEmpty() && DownloadProcessLimit > 0 && MaxQueueProcesses > 0)
	{
		const FAssetDDCRequest* PendingRequest = ProcessDDCQueue.Peek();
		if (GetDerivedDataCacheRef().PollAsynchronousCompletion(PendingRequest->DDCHandle))
		{
			bool bDataWasBuilt;

			TArray<uint8> OutContent;
			bool bGetSuccessful = GetDerivedDataCacheRef().GetAsynchronousResults(PendingRequest->DDCHandle, OutContent, &bDataWasBuilt);
			if (bGetSuccessful)
			{
				LoadDDCContentIntoDatabase(PendingRequest->AssetData, OutContent, PendingRequest->DDCKey);
				DownloadProcessLimit--;
			}
			else if (UserSettings->bShowAssetsNeedingIndexing)
			{
				AssetNeedingReindexing.Add(PendingRequest->AssetData);
			}

			ProcessDDCQueue.Pop();
			ActiveDownloads--;
			MaxQueueProcesses--;
			continue;
		}
		break;
	}

	if (bDatabaseOpen && ((FPlatformTime::Seconds() - LastRecordCountUpdateSeconds) > 30))
	{
		LastRecordCountUpdateSeconds = FPlatformTime::Seconds();

		ImmediateOperations.Enqueue([this]() {
			FScopeLock ScopedLock(&SearchDatabaseCS);
			TotalSearchRecords = SearchDatabase.GetTotalSearchRecords();
		});
	}

	return true;
}

uint32 FAssetSearchManager::Run()
{
	Tick_DatabaseOperationThread();
	return 0;
}

void FAssetSearchManager::Tick_DatabaseOperationThread()
{
	while (RunThread)
	{
		if (!bDatabaseOpen)
		{
			TryConnectToDatabase();
			FPlatformProcess::Sleep(1);
			continue;
		}

		TFunction<void()> Operation;
		if (ImmediateOperations.Dequeue(Operation) || FeedOperations.Dequeue(Operation) || UpdateOperations.Dequeue(Operation))
		{
			Operation();
		}
		else
		{
			FPlatformProcess::Sleep(0.1f);
		}
	}
}

void FAssetSearchManager::ForceIndexOnAssetsMissingIndex()
{
	FScopeLock Lock(&AssetSearchManagerCS);

	{
		FScopedSlowTask IndexingTask(AssetNeedingReindexing.Num(), LOCTEXT("ForceIndexOnAssetsMissingIndex", "Indexing Assets"));
		IndexingTask.MakeDialog(true);

		int32 RemovedCount = 0;

		TArray<FAssetData> RedirectorsWithBrokenMetadata;

		TGuardValue<bool> GuardResetTesting(GIsAutomationTesting, true);

		const int32 OnePercentChunk = (int32)(AssetNeedingReindexing.Num() / 100.0);
		int32 ChunkProgress = 0;

		FUnloadPackageScope UnloadScope;
		for (const FAssetData& Asset : AssetNeedingReindexing)
		{
			if (IndexingTask.ShouldCancel())
			{
				break;
			}

			if (RemovedCount > ChunkProgress)
			{
				ChunkProgress += OnePercentChunk;
				IndexingTask.EnterProgressFrame(OnePercentChunk, FText::Format(LOCTEXT("ForceIndexOnAssetsMissingIndexFormat", "Indexing Asset ({0} of {1})"), RemovedCount + 1, AssetNeedingReindexing.Num()));
			}

			if (UObject* AssetToIndex = Asset.GetAsset())
			{
				// This object's metadata incorrectly labled it as something other than a redirector.  We need to resave it
				// to stop it from appearing as something it's not.
				if (UObjectRedirector* Redirector = Cast<UObjectRedirector>(AssetToIndex))
				{
					RedirectorsWithBrokenMetadata.Add(Asset);
					RemovedCount++;
					continue;
				}

				if (!bTryIndexAssetsOnLoad)
				{
					StoreIndexForAsset(AssetToIndex);
				}
			}

			if (UnloadScope.GetObjectsLoaded() > 2000)
			{
				UnloadScope.TryUnload(true);
			}

			RemovedCount++;
		}

		if (RedirectorsWithBrokenMetadata.Num() > 0)
		{
			EAppReturnType::Type ResaveRedirectors = FMessageDialog::Open(EAppMsgType::YesNo,
				LOCTEXT("ResaveRedirectors", "We found some redirectors that didn't have the correct asset metadata identifying them as redirectors.  Would you like to resave them, so that they stop appearing as missing asset indexes?"));

			if (ResaveRedirectors == EAppReturnType::Yes)
			{
				TArray<UPackage*> PackagesToSave;
				for (const FAssetData& BrokenAsset : RedirectorsWithBrokenMetadata)
				{
					if (UObject* Redirector = BrokenAsset.GetAsset())
					{
						PackagesToSave.Add(Redirector->GetOutermost());
					}
				}

				FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, /*bCheckDirty*/false, /*bPromptToSave*/false);
			}
		}

		AssetNeedingReindexing.RemoveAtSwap(0, RemovedCount);
	}

	if (IntermediateStorage == ESearchIntermediateStorage::AssetTagData)
	{
		const bool bPromptUserToSave = true;
		const bool bSaveMapPackages = true;
		const bool bSaveContentPackages = true;
		const bool bFastSave = false;
		const bool bNotifyNoPackagesSaved = false;
		const bool bCanBeDeclined = false;
		FEditorFileUtils::SaveDirtyPackages(bPromptUserToSave, bSaveMapPackages, bSaveContentPackages, bFastSave, bNotifyNoPackagesSaved, bCanBeDeclined);
	}
}

void FAssetSearchManager::Search(FSearchQueryPtr SearchQuery)
{
	check(IsInGameThread());
	FScopeLock Lock(&AssetSearchManagerCS);

	if (FStudioTelemetry::IsAvailable())
	{
		FStudioTelemetry::Get().RecordEvent(TEXT("AssetSearch"), { FAnalyticsEventAttribute(TEXT("QueryString"), SearchQuery->QueryText)} );
	}
	
	ImmediateOperations.Enqueue([this, SearchQuery]() {

		TArray<FSearchRecord> Results;

		{
			FScopeLock ScopedLock(&SearchDatabaseCS);
			// This short circuits the search if it's no longer important to fulfill
			if (SearchQuery->IsQueryStillImportant())
			{
				SearchDatabase.EnumerateSearchResults(SearchQuery->QueryText, [&Results](FSearchRecord&& InResult) {
					Results.Add(MoveTemp(InResult));
					return true;
				});
			}
		}

		AsyncMainThreadTask([ResultsFwd = MoveTemp(Results), SearchQuery]() mutable {
			if (FSearchQuery::ResultsCallbackFunction ResultsCallback = SearchQuery->GetResultsCallback())
			{
				ResultsCallback(MoveTemp(ResultsFwd));
			}
		});
	});

	for (auto& SearchProviderEntry : SearchProviders)
	{
		TUniquePtr<ISearchProvider>& Provider = SearchProviderEntry.Value;
		Provider->Search(SearchQuery);
	}
}

void FAssetSearchManager::AsyncMainThreadTask(TFunction<void()> Task)
{
	FScopeLock Lock(&AssetSearchManagerCS);
	GT_Tasks.Enqueue(Task);
}

void FAssetSearchManager::ProcessGameThreadTasks()
{
	FScopeLock Lock(&AssetSearchManagerCS);
	if (!GT_Tasks.IsEmpty())
	{
		if (UE::IsSavingPackage())
		{
			// If we're saving packages just give up, the call in Tick_GameThread will do this later.
			return;
		}

		int MaxGameThreadTasksPerTick = 1000;

		TFunction<void()> Operation;
		while (GT_Tasks.Dequeue(Operation) && MaxGameThreadTasksPerTick > 0)
		{
			Operation();
			MaxGameThreadTasksPerTick--;
		}
	}
}

#undef LOCTEXT_NAMESPACE
