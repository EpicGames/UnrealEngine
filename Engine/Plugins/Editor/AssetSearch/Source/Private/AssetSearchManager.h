// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "AssetSearchDatabase.h"
#include "FileInfoDatabase.h"
#include "Containers/Queue.h"
#include "Containers/Ticker.h"
#include "HAL/Runnable.h"
#include "SearchQuery.h"
#include "UObject/WeakObjectPtr.h"

class IAssetIndexer;
struct FSearchStats;

class FObjectPostSaveContext;
class FRunnableThread;
class UClass;
class ISearchProvider;
enum class ESearchIntermediateStorage : uint8;

class FAssetSearchManager : public FRunnable
{
	static const FName AssetSearchIndexVersionTag;
	static const FName AssetSearchIndexHashTag;
	static const FName AssetSearchIndexDataTag;

public:
	FAssetSearchManager();
	virtual ~FAssetSearchManager();
	
	void Start();
	void RegisterAssetIndexer(const UClass* AssetClass, TUniquePtr<IAssetIndexer>&& Indexer);
	void RegisterSearchProvider(FName SearchProviderName, TUniquePtr<ISearchProvider>&& InSearchProvider);

	FSearchStats GetStats() const;

	void Search(FSearchQueryPtr SearchQuery);

	// Utility
	void ForceIndexOnAssetsMissingIndex();

private:
	void TryConnectToDatabase();

	bool Tick_GameThread(float DeltaTime);
	virtual uint32 Run() override;
	void Tick_DatabaseOperationThread();

private:
	void UpdateScanningAssets();
	void StartScanningAssets();
	void StopScanningAssets();

	void OnAssetAdded(const FAssetData& InAssetData);
	void OnAssetRemoved(const FAssetData& InAssetData);
	void StartScanningAssetsOnAssetScanFinished();
	TArray<FAssetData> StartScanningAssetsOnAssetScanFinishedInternal();

	void HandleOnGetExtraObjectTags(FAssetRegistryTagsContext Context);
	void HandlePackageSaved(const FString& PackageFilename, UPackage* Package, FObjectPostSaveContext ObjectSaveContext);
	void OnAssetLoaded(UObject* InObject);

	void AddOrUpdateAsset(const FAssetData& InAsset, const FString& IndexedJson, const FString& DerivedDataKey);

	bool RequestIndexAsset_DDC(const UObject* InAsset);
	bool IsAssetIndexable(const UObject* InAsset) const;
	
	bool TryLoadIndexForAsset(const FAssetData& InAsset);
	bool TryLoadIndexForAsset_Tags(const FAssetData& InAssetData);
	bool TryLoadIndexForAsset_DDC(const FAssetData& InAssetData);

	void AsyncRequestDownload(const FAssetData& InAssetData, const FString& InDDCKey);
	bool AsyncGetDerivedDataKey(const FAssetData& UnindexedAsset, TFunction<void(bool, FString)> DDCKeyCallback);
	bool HasIndexerForClass(const UClass* InAssetClass) const;

	FString GetBaseIndexKey(const UClass* InAssetClass) const;
	FString GetIndexerVersion(const UClass* InAssetClass) const;
	bool IndexAsset(const FAssetData& InAssetData, const UObject* InAsset, FString& OutIndexedJson) const;
	
	void StoreIndexForAsset(const UObject* InAsset);
	void StoreIndexForAsset_Tags(const UObject* InAsset);
	void StoreIndexForAsset_DDC(const UObject* InAsset);

	void LoadDDCContentIntoDatabase(const FAssetData& InAsset, const TArray<uint8>& Content, const FString& DerivedDataKey);

	void AsyncMainThreadTask(TFunction<void()> Task);
	void ProcessGameThreadTasks();

	uint64 GetTextHash(FStringView PackageRelativeExportPath) const;

private:
	bool bStarted = false;
	bool bFinishedOnce = false;

private:
	mutable FCriticalSection AssetSearchManagerCS;
	FFileInfoDatabase FileInfoDatabase;
	FCriticalSection FileInfoDatabaseCS;
	FAssetSearchDatabase SearchDatabase;
	FCriticalSection SearchDatabaseCS;
	TAtomic<int32> PendingDatabaseUpdates;
	TAtomic<int32> IsAssetUpToDateCount;
	TAtomic<int32> ActiveDownloads;
	TAtomic<int32> DownloadQueueCount;
	TAtomic<int64> TotalSearchRecords;

	double LastRecordCountUpdateSeconds;

private:
	TMap<FName, TUniquePtr<IAssetIndexer>> Indexers;

	TMap<FName, TUniquePtr<ISearchProvider>> SearchProviders;

	TArray<TWeakObjectPtr<UObject>> RequestIndexQueue;

	struct FAssetOperation
	{
		FAssetData Asset;
		bool bRemoval = false;
	};

	TArray<FAssetOperation> ProcessAssetQueue;

	struct FAssetDDCRequest
	{
		FAssetData AssetData;
		FString DDCKey;
		uint32 DDCHandle;
	};
	TQueue<FAssetDDCRequest, EQueueMode::Mpsc> DownloadQueue;
	TQueue<FAssetDDCRequest, EQueueMode::Mpsc> ProcessDDCQueue;

	TArray<FAssetData> AssetNeedingReindexing;

	FTSTicker::FDelegateHandle TickerHandle;

	TQueue<TFunction<void()>, EQueueMode::Mpsc> GT_Tasks;

private:
	volatile bool bDatabaseOpen = false;
	volatile double LastConnectionAttempt = 0;

	ESearchIntermediateStorage IntermediateStorage;

	TAtomic<bool> RunThread;
	FRunnableThread* DatabaseThread = nullptr;

	TQueue<TFunction<void()>, EQueueMode::Mpsc> ImmediateOperations;
	TQueue<TFunction<void()>, EQueueMode::Mpsc> FeedOperations;
	TQueue<TFunction<void()>, EQueueMode::Mpsc> UpdateOperations;
};
