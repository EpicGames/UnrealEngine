// Copyright Epic Games, Inc. All Rights Reserved.

#include "IAssetSearchModule.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SSearchBrowser.h"
#include "AssetSearchCommands.h"
#include "AssetSearchManager.h"
#include "ContentBrowserDelegates.h"
#include "ContentBrowserModule.h"
#include "LevelEditor.h"
#include "Framework/Application/SlateApplication.h"

#define LOCTEXT_NAMESPACE "FAssetSearchModule"

static const FName SearchTabName("Search");

class FAssetSearchModule : public IAssetSearchModule
{
public:

	// IModuleInterface interface
	
	virtual void StartupModule() override
	{
		SearchManager = MakeUnique<FAssetSearchManager>();
		SearchManager->Start();

		if((GIsEditor && !IsRunningCommandlet()))
		{
			FAssetSearchCommands::Register();
			
			FGlobalTabmanager::Get()->RegisterNomadTabSpawner(SearchTabName, FOnSpawnTab::CreateRaw(this, &FAssetSearchModule::HandleSpawnSettingsTab))
				.SetDisplayName(LOCTEXT("Search", "Search"))
				.SetTooltipText(LOCTEXT("SearchTab", "Search Tab"))
				.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "Symbols.SearchGlass"))
				.SetGroup(WorkspaceMenu::GetMenuStructure().GetToolsCategory());
				
			// Register content browser hook
			FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
		
			TArray<FContentBrowserCommandExtender>& CBCommandExtenderDelegates = ContentBrowserModule.GetAllContentBrowserCommandExtenders();
			CBCommandExtenderDelegates.Add(FContentBrowserCommandExtender::CreateRaw(this, &FAssetSearchModule::OnExtendContentBrowserCommands));
			ContentBrowserCommandExtenderDelegateHandle = CBCommandExtenderDelegates.Last().GetHandle();
			
			// Register level editor hooks and commands
			FLevelEditorModule& LevelEditorModule = FModuleManager::Get().LoadModuleChecked<FLevelEditorModule>("LevelEditor");
			TSharedRef<FUICommandList> CommandList = LevelEditorModule.GetGlobalLevelEditorActions();
			
			CommandList->MapAction(FAssetSearchCommands::Get().ViewAssetSearch, 
			FExecuteAction::CreateLambda([this]
			{
				ExecuteOpenObjectBrowser();
			}));
		}
	}

	virtual void ShutdownModule() override
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(SearchTabName);

		if ((GIsEditor && !IsRunningCommandlet()) && UObjectInitialized() && FSlateApplication::IsInitialized())
		{
			FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
		
			TArray<FContentBrowserCommandExtender>& CBCommandExtenderDelegates = ContentBrowserModule.GetAllContentBrowserCommandExtenders();
			CBCommandExtenderDelegates.RemoveAll([this](const FContentBrowserCommandExtender& Delegate) { return Delegate.GetHandle() == ContentBrowserCommandExtenderDelegateHandle; });
			
			FLevelEditorModule& LevelEditorModule = FModuleManager::Get().LoadModuleChecked<FLevelEditorModule>("LevelEditor");

			TSharedRef<FUICommandList> CommandList = LevelEditorModule.GetGlobalLevelEditorActions();
			CommandList->UnmapAction(FAssetSearchCommands::Get().ViewAssetSearch);
		}
		
		SearchManager.Reset();
	}

	void ExecuteOpenObjectBrowser()
	{
		FGlobalTabmanager::Get()->TryInvokeTab(SearchTabName);
	}

	void OnExtendContentBrowserCommands(TSharedRef<FUICommandList> CommandList, FOnContentBrowserGetSelection GetSelectionDelegate)
	{
		CommandList->MapAction(FAssetSearchCommands::Get().ViewAssetSearch,
			FExecuteAction::CreateLambda([this, GetSelectionDelegate]
		{
			FGlobalTabmanager::Get()->TryInvokeTab(SearchTabName);
		})
		);
	}
	
	TSharedRef<SDockTab> HandleSpawnSettingsTab(const FSpawnTabArgs& SpawnTabArgs)
	{
		const TSharedRef<SDockTab> DockTab = SNew(SDockTab)
			.TabRole(ETabRole::NomadTab);

		DockTab->SetContent(SNew(SSearchBrowser));

		return DockTab;
	}

	virtual FSearchStats GetStats() const override
	{
		return SearchManager->GetStats();
	}

	virtual void Search(FSearchQueryPtr SearchQuery) override
	{
		SearchManager->Search(SearchQuery);
	}

	virtual void ForceIndexOnAssetsMissingIndex() override
	{
		SearchManager->ForceIndexOnAssetsMissingIndex();
	}

	virtual void RegisterAssetIndexer(const UClass* InAssetClass, TUniquePtr<IAssetIndexer>&& Indexer) override
	{
		SearchManager->RegisterAssetIndexer(InAssetClass, MoveTemp(Indexer));
	}

	virtual void RegisterSearchProvider(FName SearchProviderName, TUniquePtr<ISearchProvider>&& InSearchProvider) override
	{
		SearchManager->RegisterSearchProvider(SearchProviderName, MoveTemp(InSearchProvider));
	}

private:
	TUniquePtr<FAssetSearchManager> SearchManager;
	
    FDelegateHandle ContentBrowserCommandExtenderDelegateHandle;
};

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FAssetSearchModule, AssetSearch);
