// Copyright Epic Games, Inc. All Rights Reserved.

#include "FileInfoDatabase.h"

#include "HAL/PlatformFile.h"
#include "SQLiteDatabase.h"
#include "HAL/FileManager.h"

#include "AssetRegistry/AssetData.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"
#include "HAL/PlatformFileManager.h"
#include "SQLitePreparedStatement.h"

DECLARE_LOG_CATEGORY_CLASS(LogFileInfo, Log, All);

enum class EFileInfoDatabaseVersion
{
	Empty,
	Initial,

	// -----<new versions can be added above this line>-------------------------------------------------
	VersionPlusOne,
	LatestVersion = VersionPlusOne - 1
};

class FFileInfoDatabaseStatements
{
private:
	FSQLiteDatabase& Database;

public:
	explicit FFileInfoDatabaseStatements(FSQLiteDatabase& InDatabase)
		: Database(InDatabase)
	{
		check(Database.IsValid());
	}

	bool CreatePreparedStatements()
	{
		check(Database.IsValid());

#define PREPARE_STATEMENT(VAR)																				\
			(VAR) = Database.PrepareStatement<decltype(VAR)>(ESQLitePreparedStatementFlags::Persistent);	\
			if (!(VAR).IsValid()) { return false; }

		PREPARE_STATEMENT(Statement_BeginTransaction);
		PREPARE_STATEMENT(Statement_CommitTransaction);
		PREPARE_STATEMENT(Statement_RollbackTransaction);

		PREPARE_STATEMENT(Statement_AddFileInfo);
		PREPARE_STATEMENT(Statement_UpdateFileInfo);
		PREPARE_STATEMENT(Statement_GetFileInfo);
		PREPARE_STATEMENT(Statement_GetAllFileInfos);

#undef PREPARE_STATEMENT

		return true;
	}

	/**
	 * Statements managing database transactions
	 */

	 /** Begin a database transaction */
	SQLITE_PREPARED_STATEMENT_SIMPLE(FBeginTransaction, "BEGIN TRANSACTION;");
	FBeginTransaction Statement_BeginTransaction;
	bool BeginTransaction()
	{
		return Statement_BeginTransaction.Execute();
	}

	/** Commit a database transaction */
	SQLITE_PREPARED_STATEMENT_SIMPLE(FCommitTransaction, "COMMIT TRANSACTION;");
	FCommitTransaction Statement_CommitTransaction;
	bool CommitTransaction()
	{
		return Statement_CommitTransaction.Execute();
	}

	/** Rollback a database transaction */
	SQLITE_PREPARED_STATEMENT_SIMPLE(FRollbackTransaction, "ROLLBACK TRANSACTION;");
	FRollbackTransaction Statement_RollbackTransaction;
	bool RollbackTransaction()
	{
		return Statement_RollbackTransaction.Execute();
	}

	/**
	 * Application Statements
	 */

	struct FCachedFileInfo
	{
		int64 FileId;
		FString FilePath;
		FDateTime LastModifed;
		FString Hash;

		FAssetFileInfo ToAssetFileInfo() const
		{
			FAssetFileInfo AssetFileInfo;
			AssetFileInfo.LastModified = LastModifed;
			AssetFileInfo.PackageName = *FilePath;
			LexFromString(AssetFileInfo.Hash, *Hash);

			return AssetFileInfo;
		}
	};

	SQLITE_PREPARED_STATEMENT(FGetFileInfo,
		"SELECT fileid, file_last_modified, file_hash FROM table_files WHERE file_path = ?1;",
		SQLITE_PREPARED_STATEMENT_COLUMNS(int64, FDateTime, FString),
		SQLITE_PREPARED_STATEMENT_BINDINGS(FString)
	);
	private: FGetFileInfo Statement_GetFileInfo;
	public: bool GetFileInfo(const FString& InFullFilePath, FCachedFileInfo& OutFileInfo)
	{
		OutFileInfo.FilePath = InFullFilePath.ToLower();
		if (Statement_GetFileInfo.BindAndExecuteSingle(OutFileInfo.FilePath, OutFileInfo.FileId, OutFileInfo.LastModifed, OutFileInfo.Hash))
		{
			return true;
		}

		return false;
	}

	SQLITE_PREPARED_STATEMENT(FGetAllFileInfos,
		"SELECT file_path, file_last_modified, file_hash FROM table_files;",
		SQLITE_PREPARED_STATEMENT_COLUMNS(FString, FDateTime, FString),
		SQLITE_PREPARED_STATEMENT_BINDINGS()
	);
	FGetAllFileInfos Statement_GetAllFileInfos;
	bool GetAllFileInfos(TFunctionRef<ESQLitePreparedStatementExecuteRowResult(FAssetFileInfo&&)> InCallback)
	{
		return Statement_GetAllFileInfos.BindAndExecute([&InCallback](const FGetAllFileInfos& InStatement) //-V562
		{
			FCachedFileInfo FileInfo;
			if (InStatement.GetColumnValues(FileInfo.FilePath, FileInfo.LastModifed, FileInfo.Hash))
			{
				return InCallback(FileInfo.ToAssetFileInfo());
			}
			return ESQLitePreparedStatementExecuteRowResult::Error;
		}) != INDEX_NONE;
	}

	SQLITE_PREPARED_STATEMENT_BINDINGS_ONLY(
		FUpdateFileInfo,
		" UPDATE table_files SET file_last_modified = ?2, file_hash = ?3 WHERE file_path = ?1;",
		SQLITE_PREPARED_STATEMENT_BINDINGS(FString, FDateTime, FString)
	);
	private: FUpdateFileInfo Statement_UpdateFileInfo;
	SQLITE_PREPARED_STATEMENT_BINDINGS_ONLY(
		FAddFileInfo,
		" INSERT INTO table_files(file_path, file_last_modified, file_hash)"
		" VALUES(?1, ?2, ?3);",
		SQLITE_PREPARED_STATEMENT_BINDINGS(FString, FDateTime, FString)
	);
	private: FAddFileInfo Statement_AddFileInfo;
	public: bool AddOrUpdateFileInfo(const FAssetData& InAssetData, FAssetFileInfo& OutFileInfo)
	{
		const FString PackageName = InAssetData.PackageName.ToString();
		const FString Extension = (InAssetData.PackageFlags & PKG_ContainsMap) ? FPackageName::GetMapPackageExtension() : FPackageName::GetAssetPackageExtension();

		FString FilePath;
		if (!FPackageName::TryConvertLongPackageNameToFilename(PackageName, FilePath, Extension))
		{
			return false;
		}

		const FString FullFilePath = FPaths::ConvertRelativePathToFull(FilePath);

		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		const FDateTime CurrentLastModified = PlatformFile.GetTimeStamp(*FullFilePath);

		FCachedFileInfo FileInfo;
		if (GetFileInfo(PackageName.ToLower(), FileInfo))
		{
			if (CurrentLastModified == FileInfo.LastModifed)
			{
				OutFileInfo = FileInfo.ToAssetFileInfo();
				return false;
			}

			OutFileInfo = FileInfo.ToAssetFileInfo();
			OutFileInfo.LastModified = CurrentLastModified;
			OutFileInfo.Hash = FMD5Hash::HashFile(*FullFilePath);
			Statement_UpdateFileInfo.BindAndExecuteSingle(PackageName.ToLower(), OutFileInfo.LastModified, LexToString(OutFileInfo.Hash));
			return true;
		}
		else
		{
			OutFileInfo = FileInfo.ToAssetFileInfo();
			OutFileInfo.LastModified = CurrentLastModified;
			OutFileInfo.Hash = FMD5Hash::HashFile(*FullFilePath);
			Statement_AddFileInfo.BindAndExecuteSingle(PackageName.ToLower(), OutFileInfo.LastModified, LexToString(OutFileInfo.Hash));
			return true;
		}
	}
};

class FFileInfoDatabaseScopedTransaction
{
public:
	explicit FFileInfoDatabaseScopedTransaction(FFileInfoDatabaseStatements& InStatements)
		: Statements(InStatements)
		, bHasTransaction(Statements.BeginTransaction()) // This will fail if a transaction is already open
	{
	}

	~FFileInfoDatabaseScopedTransaction()
	{
		Commit();
	}

	bool CommitOrRollback(const bool bShouldCommit)
	{
		if (bShouldCommit)
		{
			Commit();
			return true;
		}

		Rollback();
		return false;
	}

	void Commit()
	{
		if (bHasTransaction)
		{
			verify(Statements.CommitTransaction());
			bHasTransaction = false;
		}
	}

	void Rollback()
	{
		if (bHasTransaction)
		{
			verify(Statements.RollbackTransaction());
			bHasTransaction = false;
		}
	}

private:
	FFileInfoDatabaseStatements& Statements;
	bool bHasTransaction;
};

FFileInfoDatabase::FFileInfoDatabase()
	: Database(MakeUnique<FSQLiteDatabase>())
	, DatabaseFileName(TEXT("FileInfo.db"))
{
}

FFileInfoDatabase::~FFileInfoDatabase()
{
	Close();
}

bool FFileInfoDatabase::IsValid() const
{
	return Database->IsValid();
}

bool FFileInfoDatabase::Open(const FString& InSessionPath)
{
	return Open(InSessionPath, ESQLiteDatabaseOpenMode::ReadWriteCreate);
}

bool FFileInfoDatabase::Open(const FString& InSessionPath, const ESQLiteDatabaseOpenMode InOpenMode)
{
	SessionPath = InSessionPath;
	
	if (Database->IsValid())
	{
		return false;
	}

	if (!Database->Open(*(InSessionPath / DatabaseFileName), ESQLiteDatabaseOpenMode::ReadWriteCreate))
	{
		UE_LOGF(LogFileInfo, Error, "Failed to open database for '%ls': %ls", *InSessionPath, *GetLastError());
		return false;
	}

	if (!Database->PerformQuickIntegrityCheck())
	{
		UE_LOGF(LogFileInfo, Error, "Database failed integrity check, deleting.");

		const bool bDeleteTheDatabase = true;
		Close(bDeleteTheDatabase);

		return false;
	}

	// Set the database to use exclusive WAL mode for performance (exclusive works even on platforms without a mmap implementation)
	// Set the database "NORMAL" fsync mode to only perform a fsync when check-pointing the WAL to the main database file (fewer fsync calls are better for performance, with a very slight loss of WAL durability if the power fails)
	Database->Execute(TEXT("PRAGMA journal_mode=WAL;"));
	Database->Execute(TEXT("PRAGMA synchronous=FULL;"));
	Database->Execute(TEXT("PRAGMA cache_size=1000;"));
	Database->Execute(TEXT("PRAGMA page_size=65535;"));
	Database->Execute(TEXT("PRAGMA locking_mode=EXCLUSIVE;"));

	int32 LoadedDatabaseVersion = 0;
	Database->GetUserVersion(LoadedDatabaseVersion);
	if (LoadedDatabaseVersion != (int32)EFileInfoDatabaseVersion::Empty)
	{
		if (LoadedDatabaseVersion > (int32)EFileInfoDatabaseVersion::LatestVersion)
		{
			Close();
			UE_LOGF(LogFileInfo, Error, "Failed to open database for '%ls': Database is too new (version %d, expected = %d)", *InSessionPath, LoadedDatabaseVersion, (int32)EFileInfoDatabaseVersion::LatestVersion);
			return false;
		}
		else if (LoadedDatabaseVersion < (int32)EFileInfoDatabaseVersion::LatestVersion)
		{
			Close(true);
			UE_LOGF(LogFileInfo, Log, "Opened database '%ls': Database is too old (version %d, expected = %d), creating new database", *InSessionPath, LoadedDatabaseVersion, (int32)EFileInfoDatabaseVersion::LatestVersion);
			return Open(InSessionPath, InOpenMode);
		}
	}

	// Create our required tables
	//========================================================================
	if (!ensure(Database->Execute(TEXT("CREATE TABLE IF NOT EXISTS table_files(fileid INTEGER PRIMARY KEY, file_path TEXT UNIQUE, file_last_modified INTEGER NOT NULL, file_hash);"))))
	{
		LogLastError();
		Close();
		return false;
	}

	if (!ensure(Database->Execute(
		TEXT("CREATE UNIQUE INDEX IF NOT EXISTS file_path_index ON table_files(file_path);")
	)))
	{
		LogLastError();
		Close();
		return false;
	}

	// The database will have the latest schema at this point, so update the user-version
	if (!Database->SetUserVersion((int32)EFileInfoDatabaseVersion::LatestVersion))
	{
		Close();
		return false;
	}

	// Create our required prepared statements
	Statements = MakeUnique<FFileInfoDatabaseStatements>(*Database);
	if (!ensure(Statements->CreatePreparedStatements()))
	{
		Close();
		return false;
	}

	if (!Database->PerformQuickIntegrityCheck())
	{
		UE_LOGF(LogFileInfo, Error, "Database failed integrity check, deleting.");

		const bool bDeleteTheDatabase = true;
		Close(bDeleteTheDatabase);

		return false;
	}

	return true;
}

bool FFileInfoDatabase::Close(const bool InDeleteDatabase)
{
	if (!Database->IsValid())
	{
		return false;
	}

	// Need to destroy prepared statements before the database can be closed
	Statements.Reset();

	if (!Database->Close())
	{
		UE_LOGF(LogFileInfo, Error, "Failed to close database for '%ls': %ls", *SessionPath, *GetLastError());
		return false;
	}

	if (InDeleteDatabase)
	{
		IFileManager::Get().Delete(*(SessionPath / DatabaseFileName), false);
	}

	SessionPath.Reset();

	return true;
}

FString FFileInfoDatabase::GetFilename() const
{
	return Database->GetFilename();
}

FString FFileInfoDatabase::GetLastError() const
{
	return Database->GetLastError();
}

void FFileInfoDatabase::LogLastError() const
{
	UE_LOGF(LogFileInfo, Error, "Database '%ls' Error: %ls", *SessionPath, *GetLastError());
}

bool FFileInfoDatabase::AddOrUpdateFileInfo(const FAssetData& InAssetData, FAssetFileInfo& OutFileInfo)
{
	if (ensure(Statements))
	{
		return Statements->AddOrUpdateFileInfo(InAssetData, OutFileInfo);
	}

	return false;
}

void FFileInfoDatabase::AddOrUpdateFileInfos(const TArray<FAssetData>& InAssets)
{
	for (const FAssetData& InAsset : InAssets)
	{
		// If it's a redirector act like it has been removed from the system,
		// we don't want old duplicate entries for it.
		if (InAsset.IsRedirector())
		{
			continue;
		}

		// Freshen hash cache
		FAssetFileInfo FileInfo;
		AddOrUpdateFileInfo(InAsset, FileInfo);
	}
}

TMap<FName, FAssetFileInfo> FFileInfoDatabase::GetAllFileInfos()
{
	TMap<FName, FAssetFileInfo> FileInfos;
	Statements->GetAllFileInfos([&FileInfos](FAssetFileInfo&& InResult)
	{
		FileInfos.Add(InResult.PackageName, InResult);

		return ESQLitePreparedStatementExecuteRowResult::Continue;
	});

	return FileInfos;
}

//"database disk image is malformed"
