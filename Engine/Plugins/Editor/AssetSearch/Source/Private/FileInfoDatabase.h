// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/DateTime.h"
#include "Misc/SecureHash.h"

class FSQLiteDatabase;
enum class ESQLiteDatabaseOpenMode : uint8;
class FFileInfoDatabaseStatements;
struct FAssetData;

struct FAssetFileInfo
{
	FName PackageName;
	FDateTime LastModified;
	FMD5Hash Hash;
};

class FFileInfoDatabase
{
public:
	FFileInfoDatabase();
	~FFileInfoDatabase();

	FFileInfoDatabase(const FFileInfoDatabase&) = delete;
	FFileInfoDatabase& operator=(const FFileInfoDatabase&) = delete;

	FFileInfoDatabase(FFileInfoDatabase&&) = default;
	FFileInfoDatabase& operator=(FFileInfoDatabase&&) = default;

	/**
	 * Is this a valid database? (ie, has been successfully opened).
	 */
	bool IsValid() const;

	/**
	 * Open (or create) a database file.
	 *
	 * @param InSessionPath				The root path to store all the data for this session under.
	 *
	 * @return True if the database was opened, false otherwise.
	 */
	bool Open(const FString& InSessionPath);

	/**
	 * Open (or create) a database file.
	 *
	 * @param InSessionPath				The root path to store all the data for this session under.
	 * @param InOpenMode				How should the database be opened?
	 *
	 * @return True if the database was opened, false otherwise.
	 */
	bool Open(const FString& InSessionPath, const ESQLiteDatabaseOpenMode InOpenMode);

	/**
	 * Close an open database file.
	 *
	 * @param InDeleteDatabase			True if the session database and its associated data should be deleted after closing the database.
	 */
	bool Close(const bool InDeleteDatabase = false);

	/**
	 * Get the filename of the currently open database, or an empty string.
	 * @note The returned filename will be an absolute pathname.
	 */
	FString GetFilename() const;

	/**
	 * Get the last error reported by this database.
	 */
	FString GetLastError() const;

	/**
	 * Update the file hash table.
	 */
	bool AddOrUpdateFileInfo(const FAssetData& InAssetData, FAssetFileInfo& OutFileInfo);
	void AddOrUpdateFileInfos(const TArray<FAssetData>& InAssets);

	void UpdateFileHash(const FAssetData& InAssetData, FAssetFileInfo& OutFileInfo);

	TMap<FName, FAssetFileInfo> GetAllFileInfos();
	
	bool bEnableIntegrityChecks = false;

private:
	void LogLastError() const;

private:
	/** Internal SQLite database */
	TUniquePtr<FSQLiteDatabase> Database;

	/** The database filename name we use */
	FString DatabaseFileName;

	/** Root path to store all session data under */
	FString SessionPath;

	/** Prepared statements for the currently open database */
	TUniquePtr<FFileInfoDatabaseStatements> Statements;
};
