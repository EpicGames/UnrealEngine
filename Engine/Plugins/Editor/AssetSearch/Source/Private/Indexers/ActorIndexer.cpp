// Copyright Epic Games, Inc. All Rights Reserved.

#include "ActorIndexer.h"
#include "Utility/IndexerUtilities.h"
#include "GameFramework/Actor.h"
#include "SearchSerializer.h"

enum class EActorIndexerVersion
{
	Empty,
	Initial,

	// -----<new versions can be added above this line>-------------------------------------------------
	VersionPlusOne,
	LatestVersion = VersionPlusOne - 1
};

int32 FActorIndexer::GetVersion() const
{
	return (int32)EActorIndexerVersion::LatestVersion;
}

void FActorIndexer::IndexAsset(const UObject* InAssetObject, FSearchSerializer& Serializer) const
{
	const AActor* Actor = CastChecked<AActor>(InAssetObject);
	Serializer.BeginIndexingObject(Actor, TEXT("$self"));
	FIndexerUtilities::IterateIndexableProperties(Actor, [&Serializer](const FProperty* Property, const FString& Value) {
		Serializer.IndexProperty(Property, Value);
	});
	Serializer.EndIndexingObject();
}