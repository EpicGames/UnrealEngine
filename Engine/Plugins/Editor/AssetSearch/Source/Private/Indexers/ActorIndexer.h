// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "IAssetIndexer.h"

struct FScriptContainerElement;

class FActorIndexer : public IAssetIndexer
{
	virtual FString GetName() const override { return TEXT("Actor"); }
	virtual int32 GetVersion() const override;
	virtual void IndexAsset(const UObject* InAssetObject, FSearchSerializer& Serializer) const override;
};
