// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintIndexer.h"
#include "Blueprint/BlueprintExtension.h"
#include "EdGraph/EdGraph.h"
#include "K2Node_CallFunction.h"
#include "K2Node_Variable.h"
#include "Utility/IndexerUtilities.h"
#include "K2Node_BaseMCDelegate.h"
#include "K2Node_Knot.h"
#include "EdGraphNode_Comment.h"
#include "Engine/SCS_Node.h"
#include "K2Node_Event.h"
#include "SearchSerializer.h"

#define LOCTEXT_NAMESPACE "FBlueprintIndexer"

enum class EBlueprintIndexerVersion
{
	Empty,
	Initial,
	FixingPinsToSaveValues,
	IndexingPublicEditableFieldsOnNodes,
	DontIndexPinsUnlessItsInputWithNoConnections,
	BetterSupportForIndexingEventNodes,
	BlueprintExtension,

	// -----<new versions can be added above this line>-------------------------------------------------
	VersionPlusOne,
	LatestVersion = VersionPlusOne - 1
};

int32 FBlueprintIndexer::GetVersion() const
{
	return (int32)EBlueprintIndexerVersion::LatestVersion;
}

void FBlueprintIndexer::IndexAsset(const UObject* InAssetObject, FSearchSerializer& Serializer) const
{
	const UBlueprint* BP = CastChecked<UBlueprint>(InAssetObject);

	Serializer.BeginIndexingObject(BP, TEXT("$self"));
	FIndexerUtilities::IterateIndexableProperties(BP, [&Serializer](const FProperty* Property, const FString& Value) {
		Serializer.IndexProperty(Property, Value);
	});
	Serializer.EndIndexingObject();

	IndexClassDefaultObject(BP, Serializer);
	IndexComponents(BP, Serializer);
	IndexGraphs(BP, Serializer);
	IndexExtensions(BP, Serializer);
}

void FBlueprintIndexer::IndexClassDefaultObject(const UBlueprint* InBlueprint, FSearchSerializer& Serializer) const
{
	if (UClass* GeneratedClass = InBlueprint->GeneratedClass)
	{
		if (UObject* CDO = GeneratedClass->GetDefaultObject())
		{
			Serializer.BeginIndexingObject(CDO, TEXT("Class Defaults"));
			FIndexerUtilities::IterateIndexableProperties(CDO, [&Serializer](const FProperty* Property, const FString& Value) {
				Serializer.IndexProperty(Property, Value);
			});
			Serializer.EndIndexingObject();
		}
	}
}

void FBlueprintIndexer::IndexComponents(const UBlueprint* InBlueprint, FSearchSerializer& Serializer) const
{
	if (InBlueprint->SimpleConstructionScript)
	{
		const TArray<USCS_Node*>& BPNodes = InBlueprint->SimpleConstructionScript->GetAllNodes();
		for (USCS_Node* BPNode : BPNodes)
		{
			Serializer.BeginIndexingObject(BPNode->ComponentTemplate, BPNode->GetVariableName().ToString());
			FIndexerUtilities::IterateIndexableProperties(BPNode->ComponentTemplate, [&Serializer](const FProperty* Property, const FString& Value) {
				Serializer.IndexProperty(Property, Value);
			});
			Serializer.EndIndexingObject();
		}
	}
}

void FBlueprintIndexer::IndexGraphs(const UBlueprint* InBlueprint, FSearchSerializer& Serializer) const
{
	TArray<UEdGraph*> AllGraphs;
	InBlueprint->GetAllGraphs(AllGraphs);

	for (UEdGraph* Graph : AllGraphs)
	{
		for (UEdGraphNode* Node : Graph->Nodes)
		{
			// Ignore Knots.
			if (Cast<UK2Node_Knot>(Node))
			{
				continue;
			}

			// Special rules for comment nodes
			if (UEdGraphNode_Comment* CommentNode = Cast<UEdGraphNode_Comment>(Node))
			{
				Serializer.BeginIndexingObject(Node, Node->NodeComment);
				Serializer.IndexProperty(TEXT("Comment"), Node->NodeComment);
				Serializer.EndIndexingObject();
				continue;
			}

			const FText NodeText = Node->GetNodeTitle(ENodeTitleType::MenuTitle);
			Serializer.BeginIndexingObject(Node, NodeText);
			Serializer.IndexProperty(TEXT("Title"), NodeText);

			if (!Node->NodeComment.IsEmpty())
			{
				Serializer.IndexProperty(TEXT("Comment"), Node->NodeComment);
			}

			if (UK2Node_CallFunction* FunctionNode = Cast<UK2Node_CallFunction>(Node))
			{
				IndexMemberReference(Serializer, FunctionNode->FunctionReference, TEXT("Function"));
			}
			else if (UK2Node_Event* EventNode = Cast<UK2Node_Event>(Node))
			{
				IndexMemberReference(Serializer, EventNode->EventReference, TEXT("Event"));
			}
			else if (UK2Node_BaseMCDelegate* DelegateNode = Cast<UK2Node_BaseMCDelegate>(Node))
			{
				IndexMemberReference(Serializer, DelegateNode->DelegateReference, TEXT("Delegate"));
			}
			else if (UK2Node_Variable* VariableNode = Cast<UK2Node_Variable>(Node))
			{
				IndexMemberReference(Serializer, VariableNode->VariableReference, TEXT("Variable"));
			}

			for (UEdGraphPin* Pin : Node->GetAllPins())
			{
				if (Pin->Direction == EGPD_Input && Pin->LinkedTo.Num() == 0)
				{
					const FText PinText = Pin->GetDisplayName();
					if (PinText.IsEmpty())
					{
						continue;
					}

					const FText PinValue = Pin->GetDefaultAsText();
					if (PinValue.IsEmpty())
					{
						continue;
					}

					const FString PinLabel = TEXT("[Pin] ") + *FTextInspector::GetSourceString(PinText);
					Serializer.IndexProperty(PinLabel, PinValue);
				}
			}

			// This will serialize any user exposed options for the node that are editable in the details panel.
			FIndexerUtilities::IterateIndexableProperties(Node, [&Serializer](const FProperty* Property, const FString& Value) {
				Serializer.IndexProperty(Property, Value);
			});

			Serializer.EndIndexingObject();
		}
	}
}

void FBlueprintIndexer::IndexMemberReference(FSearchSerializer& Serializer, const FMemberReference& MemberReference, const FString& MemberType) const
{
	Serializer.IndexProperty(MemberType + TEXT("Name"), MemberReference.GetMemberName());

	if (MemberReference.GetMemberGuid().IsValid())
	{
		Serializer.IndexProperty(MemberType + TEXT("Guid"), MemberReference.GetMemberGuid().ToString(EGuidFormats::Digits));
	}

	if (UClass* MemberParentClass = MemberReference.GetMemberParentClass())
	{
		Serializer.IndexProperty(MemberType + TEXT("Parent"), MemberParentClass->GetPathName());
	}
}

void FBlueprintIndexer::IndexExtensions(const UBlueprint* InBlueprint, FSearchSerializer& Serializer) const
{
	for (UBlueprintExtension* Extension : InBlueprint->GetExtensions())
	{
		Serializer.IndexNestedAsset(Extension);
	}
}

#undef LOCTEXT_NAMESPACE
