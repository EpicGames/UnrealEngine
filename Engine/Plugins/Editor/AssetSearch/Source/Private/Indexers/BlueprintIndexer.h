// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "IAssetIndexer.h"

struct FScriptContainerElement;

class UBlueprint;
struct FMemberReference;
class FSearchSerializer;
class UBlueprint;

class FBlueprintIndexer : public IAssetIndexer
{
public:
	virtual FString GetName() const override { return TEXT("Blueprint"); }
	virtual int32 GetVersion() const override;
	virtual void IndexAsset(const UObject* InAssetObject, FSearchSerializer& Serializer) const override;

private:
	void IndexClassDefaultObject(const UBlueprint* InBlueprint, FSearchSerializer& Serializer) const;
	void IndexComponents(const UBlueprint* InBlueprint, FSearchSerializer& Serializer) const;
	void IndexGraphs(const UBlueprint* InBlueprint, FSearchSerializer& Serializer) const;
	void IndexMemberReference(FSearchSerializer& Serializer, const FMemberReference& MemberReference, const FString& MemberType) const;
	void IndexExtensions(const UBlueprint* InBlueprint, FSearchSerializer& Serializer) const;
};
