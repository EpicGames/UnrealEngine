// Copyright Epic Games, Inc. All Rights Reserved.

#include "CurveTableIndexer.h"
#include "Engine/CurveTable.h"
#include "SearchSerializer.h"

enum class ECurveTableIndexerVersion
{
	Empty,
	Initial,

	// -----<new versions can be added above this line>-------------------------------------------------
	VersionPlusOne,
	LatestVersion = VersionPlusOne - 1
};

int32 FCurveTableIndexer::GetVersion() const
{
	return (int32)ECurveTableIndexerVersion::LatestVersion;
}

void FCurveTableIndexer::IndexAsset(const UObject* InAssetObject, FSearchSerializer& Serializer) const
{
	const UCurveTable* DataTable = CastChecked<UCurveTable>(InAssetObject);

	const TMap<FName, FRealCurve*>& Rows = DataTable->GetRowMap();

	Serializer.BeginIndexingObject(DataTable, TEXT("$self"));
	for (const auto& Entry : Rows)
	{
		const FName& RowName = Entry.Key;
		Serializer.IndexProperty(TEXT("Row"), RowName);
	}
	Serializer.EndIndexingObject();
}
