// Copyright Epic Games, Inc. All Rights Reserved.

#include "DataTableIndexer.h"
#include "Utility/IndexerUtilities.h"
#include "Engine/DataTable.h"
#include "SearchSerializer.h"

enum class EDataTableIndexerVersion
{
	Empty,
	Initial,
	AddedIndexingDataTableRowNames,

	// -----<new versions can be added above this line>-------------------------------------------------
	VersionPlusOne,
	LatestVersion = VersionPlusOne - 1
};

int32 FDataTableIndexer::GetVersion() const
{
	return (int32)EDataTableIndexerVersion::LatestVersion;
}

void FDataTableIndexer::IndexAsset(const UObject* InAssetObject, FSearchSerializer& Serializer) const
{
	const UDataTable* DataTable = CastChecked<UDataTable>(InAssetObject);

	const TMap<FName, uint8*>& Rows = DataTable->GetRowMap();

	Serializer.BeginIndexingObject(DataTable, TEXT("$self"));

	for (const auto& Entry : Rows)
	{
		const FName& RowName = Entry.Key;
		uint8* Row = Entry.Value;

		Serializer.IndexProperty(TEXT("Row"), RowName);

		FIndexerUtilities::IterateIndexableProperties(DataTable->GetRowStruct(), Row, [&Serializer](const FProperty* Property, const FString& Value) {
			Serializer.IndexProperty(Property, Value);
		});
	}
	Serializer.EndIndexingObject();
}