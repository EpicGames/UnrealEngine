// Copyright Epic Games, Inc. All Rights Reserved.

#include "DialogueWaveIndexer.h"
#include "Sound/DialogueWave.h"
#include "SearchSerializer.h"

enum class EDialogueWaveIndexerVersion
{
	Empty,
	Initial,

	// -----<new versions can be added above this line>-------------------------------------------------
	VersionPlusOne,
	LatestVersion = VersionPlusOne - 1
};

int32 FDialogueWaveIndexer::GetVersion() const
{
	return (int32)EDialogueWaveIndexerVersion::LatestVersion;
}

void FDialogueWaveIndexer::IndexAsset(const UObject* InAssetObject, FSearchSerializer& Serializer) const
{
	const UDialogueWave* DialogueWave = CastChecked<UDialogueWave>(InAssetObject);

	Serializer.BeginIndexingObject(DialogueWave, TEXT("$self"));

	Serializer.IndexProperty(TEXT("SpokenText"), DialogueWave->SpokenText);

	Serializer.EndIndexingObject();
}
