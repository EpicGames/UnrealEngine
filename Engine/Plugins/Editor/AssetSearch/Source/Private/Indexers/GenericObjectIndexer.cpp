// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericObjectIndexer.h"
#include "Utility/IndexerUtilities.h"
#include "SearchSerializer.h"

enum class EGenericObjectIndexerVersion
{
	Empty,
	Initial,

	// -----<new versions can be added above this line>-------------------------------------------------
	VersionPlusOne,
	LatestVersion = VersionPlusOne - 1
};

int32 FGenericObjectIndexer::GetVersion() const
{
	return (int32)EGenericObjectIndexerVersion::LatestVersion;
}

void FGenericObjectIndexer::IndexAsset(const UObject* InAssetObject, FSearchSerializer& Serializer) const
{
	Serializer.BeginIndexingObject(InAssetObject, TEXT("$self"));

	FIndexerUtilities::IterateIndexableProperties(InAssetObject, [&Serializer](const FProperty* Property, const FString& Value) {
		Serializer.IndexProperty(Property, Value);
	});

	Serializer.EndIndexingObject();
}