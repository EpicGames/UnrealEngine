// Copyright Epic Games, Inc. All Rights Reserved.

#include "LevelIndexer.h"
#include "Utility/IndexerUtilities.h"
#include "Engine/LevelScriptBlueprint.h"
#include "Engine/World.h"
#include "SearchSerializer.h"

enum class ELevelIndexerVersion
{
	Empty,
	Initial,

	// -----<new versions can be added above this line>-------------------------------------------------
	VersionPlusOne,
	LatestVersion = VersionPlusOne - 1
};

int32 FLevelIndexer::GetVersion() const
{
	return (int32)ELevelIndexerVersion::LatestVersion;
}

void FLevelIndexer::GetNestedAssetTypes(TArray<UClass*>& OutTypes) const
{
	OutTypes.Add(UBlueprint::StaticClass());
}

void FLevelIndexer::IndexAsset(const UObject* InAssetObject, FSearchSerializer& Serializer) const
{
	const UWorld* World = CastChecked<UWorld>(InAssetObject);

	if (const ULevel* Level = World->PersistentLevel)
	{
		Serializer.IndexNestedAsset(const_cast<ULevel*>(Level)->GetLevelScriptBlueprint(true));

		for (AActor* Actor : Level->Actors)
		{
			// External actors are indexed through their own package file
			if (Actor && !Actor->IsPackageExternal())
			{
				Serializer.BeginIndexingObject(Actor, Actor->GetActorLabel());
				FIndexerUtilities::IterateIndexableProperties(Actor, [&Serializer](const FProperty* Property, const FString& Value) {
					Serializer.IndexProperty(Property, Value);
				});
				Serializer.EndIndexingObject();
			}
		}
	}
}
