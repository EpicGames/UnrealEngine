// Copyright Epic Games, Inc. All Rights Reserved.

#include "MaterialExpressionIndexer.h"
#include "Utility/IndexerUtilities.h"
#include "Materials/Material.h"
#include "Materials/MaterialFunction.h"
#include "Materials/MaterialExpressionFunctionOutput.h"
#include "SearchSerializer.h"


#define LOCTEXT_NAMESPACE "FMaterialExpressionIndexer"

enum class EMaterialExpressionIndexerVersion
{
	Empty,
	Initial,

	// -----<new versions can be added above this line>-------------------------------------------------
	VersionPlusOne,
	LatestVersion = VersionPlusOne - 1
};

int32 FMaterialExpressionIndexer::GetVersion() const
{
	return (int32)EMaterialExpressionIndexerVersion::LatestVersion;
}

void FMaterialExpressionIndexer::IndexAsset(const UObject* InAssetObject, FSearchSerializer& Serializer) const
{
	Serializer.BeginIndexingObject(InAssetObject, TEXT("$self"));
	FIndexerUtilities::IterateIndexableProperties(InAssetObject, [&Serializer](const FProperty* Property, const FString& Value) {
		Serializer.IndexProperty(Property, Value);
	});
	Serializer.EndIndexingObject();
	 
	IndexParameters(InAssetObject, Serializer);
}

void FMaterialExpressionIndexer::IndexParameters(const UObject* InAssetObject, FSearchSerializer& Serializer) const
{
	const UMaterialFunction* MaterialFunction = Cast<UMaterialFunction>(InAssetObject);
	const UMaterial* Material = Cast<UMaterial>(InAssetObject);

	TConstArrayView<TObjectPtr<UMaterialExpression>> Expressions;
	Expressions = MaterialFunction ? MaterialFunction->GetExpressions() : Expressions;
	Expressions = Material ? Material->GetExpressions() : Expressions;

	for (const UMaterialExpression* Expression : Expressions)
	{
		const UMaterialExpressionFunctionOutput* ExpressionFunctionOutput = Cast<UMaterialExpressionFunctionOutput>(Expression);

		// Only index parameter and output nodes as those are unique, material nodes are typically non-unique
		if (Expression->bIsParameterExpression || ExpressionFunctionOutput)
		{
			const FText ExpressionText = FText::FromName(ExpressionFunctionOutput ? ExpressionFunctionOutput->OutputName : Expression->GetParameterName());
			Serializer.BeginIndexingObject(Expression, ExpressionText);
			Serializer.IndexProperty(Expression->GetClass()->GetFName().ToString(), ExpressionText);
			Serializer.EndIndexingObject();
		}
	}
}

#undef LOCTEXT_NAMESPACE
