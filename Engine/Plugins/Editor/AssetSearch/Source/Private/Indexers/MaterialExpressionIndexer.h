// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "IAssetIndexer.h"

struct FScriptContainerElement;

struct FMemberReference;

class FMaterialExpressionIndexer : public IAssetIndexer
{
public:
	FMaterialExpressionIndexer(const FString& InName)
		: Name(InName)
	{
	}

	virtual FString GetName() const override { return Name; }
	virtual int32 GetVersion() const override;
	virtual void IndexAsset(const UObject* InAssetObject, FSearchSerializer& Serializer) const override;

private:
	void IndexParameters(const UObject* InAssetObject, FSearchSerializer& Serializer) const;
	const FString Name;
};
