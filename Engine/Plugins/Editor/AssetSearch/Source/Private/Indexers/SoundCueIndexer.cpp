// Copyright Epic Games, Inc. All Rights Reserved.

#include "SoundCueIndexer.h"
#include "Utility/IndexerUtilities.h"
#include "Sound/SoundCue.h"
#include "EdGraph/EdGraphNode.h"
#include "SearchSerializer.h"

enum class ESoundCueIndexerVersion
{
	Empty,
	Initial,

	// -----<new versions can be added above this line>-------------------------------------------------
	VersionPlusOne,
	LatestVersion = VersionPlusOne - 1
};

int32 FSoundCueIndexer::GetVersion() const
{
	return (int32)ESoundCueIndexerVersion::LatestVersion;
}

void FSoundCueIndexer::IndexAsset(const UObject* InAssetObject, FSearchSerializer& Serializer) const
{
	const USoundCue* SoundCue = CastChecked<USoundCue>(InAssetObject);

	for (USoundNode* SoundNode : SoundCue->AllNodes)
	{
		// Apparently SoundNodes can be null...
		if (SoundNode)
		{
			if (UEdGraphNode* GraphNode = SoundNode->GetGraphNode())
			{
				const FText NodeText = GraphNode->GetNodeTitle(ENodeTitleType::MenuTitle);
				Serializer.BeginIndexingObject(SoundNode, NodeText);
				Serializer.IndexProperty(TEXT("Name"), NodeText);
				FIndexerUtilities::IterateIndexableProperties(SoundNode, [&Serializer](const FProperty* Property, const FString& Value) {
					Serializer.IndexProperty(Property, Value);
				});
				Serializer.EndIndexingObject();
			}
		}
	}
}
