// Copyright Epic Games, Inc. All Rights Reserved.

#include "WidgetBlueprintIndexer.h"
#include "WidgetBlueprint.h"
#include "Components/Widget.h"
#include "Utility/IndexerUtilities.h"
#include "SearchSerializer.h"

#define LOCTEXT_NAMESPACE "FWidgetBlueprintIndexer"

enum class EWidgetBlueprintIndexerVersion
{
	Empty,
	Initial,
	FixLabels,
	Binding,

	// -----<new versions can be added above this line>-------------------------------------------------
	VersionPlusOne,
	LatestVersion = VersionPlusOne - 1
};

int32 FWidgetBlueprintIndexer::GetVersion() const
{
	return (int32)EWidgetBlueprintIndexerVersion::LatestVersion;
}

void FWidgetBlueprintIndexer::IndexAsset(const UObject* InAssetObject, FSearchSerializer& Serializer) const
{
	const UWidgetBlueprint* BP = CastChecked<UWidgetBlueprint>(InAssetObject);

	TArray<const UWidget*> AllWidgets = BP->GetAllSourceWidgets();

	for (const UWidget* Widget : AllWidgets)
	{
		FString Label = Widget->GetDisplayLabel();
		if (Label.IsEmpty())
		{
			Label = Widget->GetName();
		}

		Serializer.BeginIndexingObject(Widget, Label);
		Serializer.IndexProperty(TEXT("Name"), Label);
		Serializer.IndexProperty(Widget->GetName() + TEXT("_Class"), Widget->GetClass()->GetName());

		FIndexerUtilities::IterateIndexableProperties(Widget, [&Serializer](const FProperty* Property, const FString& Value) {
			Serializer.IndexProperty(Property, Value);
		});

		Serializer.EndIndexingObject();
	}

	if (const UWidgetBlueprintGeneratedClass* BPClass = Cast<UWidgetBlueprintGeneratedClass>(BP->GeneratedClass))
	{
		Serializer.BeginIndexingObject(BP, TEXT("$self"));

		for (const FDelegateRuntimeBinding& Binding : BPClass->Bindings)
		{
			if (!Binding.ObjectName.IsEmpty() && !Binding.PropertyName.IsNone())
			{
				TStringBuilder<128> PropertyName;
				PropertyName.Append(TEXT("[Binding] "));
				PropertyName.Append(Binding.ObjectName);
				PropertyName.Append(TEXT("."));
				PropertyName.Append(Binding.PropertyName.ToString());

				if (Binding.SourcePath.IsValid())
				{
					Serializer.IndexProperty(PropertyName.ToString(), Binding.SourcePath.ToString());
				}
				else if (!Binding.FunctionName.IsNone())
				{
					Serializer.IndexProperty(PropertyName.ToString(), Binding.FunctionName.ToString());
				}
			}

		}

		Serializer.EndIndexingObject();
	}
}

#undef LOCTEXT_NAMESPACE
