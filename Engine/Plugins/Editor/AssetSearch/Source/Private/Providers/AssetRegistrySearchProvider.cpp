// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetRegistrySearchProvider.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "Async/Async.h"
#include "Algo/LevenshteinDistance.h"

/** Mapping of asset property tag aliases that can be used by text searches */
class FFrontendFilter_AssetPropertyTagAliases_HackCopy
{
public:
	static FFrontendFilter_AssetPropertyTagAliases_HackCopy& Get()
	{
		static FFrontendFilter_AssetPropertyTagAliases_HackCopy Singleton;
		return Singleton;
	}

	/** Get the source tag for the given asset data and alias, or none if there is no match */
	FName GetSourceTagFromAlias(const FAssetData& InAssetData, const FName InAlias)
	{
		TSharedPtr<TMap<FName, FName>>& AliasToSourceTagMapping = ClassToAliasTagsMapping.FindOrAdd(InAssetData.AssetClassPath);

		if (!AliasToSourceTagMapping.IsValid())
		{
			static const FName NAME_DisplayName(TEXT("DisplayName"));

			AliasToSourceTagMapping = MakeShared<TMap<FName, FName>>();

			UClass* AssetClass = InAssetData.GetClass();
			if (AssetClass)
			{
				TMap<FName, UObject::FAssetRegistryTagMetadata> AssetTagMetaData;
				AssetClass->GetDefaultObject()->GetAssetRegistryTagMetadata(AssetTagMetaData);

				for (const auto& AssetTagMetaDataPair : AssetTagMetaData)
				{
					if (!AssetTagMetaDataPair.Value.DisplayName.IsEmpty())
					{
						const FName DisplayName = MakeObjectNameFromDisplayLabel(AssetTagMetaDataPair.Value.DisplayName.ToString(), NAME_None);
						AliasToSourceTagMapping->Add(DisplayName, AssetTagMetaDataPair.Key);
					}
				}

				for (const auto& KeyValuePair : InAssetData.TagsAndValues)
				{
					if (FProperty* Field = FindFProperty<FProperty>(AssetClass, KeyValuePair.Key))
					{
						if (Field->HasMetaData(NAME_DisplayName))
						{
							const FName DisplayName = MakeObjectNameFromDisplayLabel(Field->GetMetaData(NAME_DisplayName), NAME_None);
							AliasToSourceTagMapping->Add(DisplayName, KeyValuePair.Key);
						}
					}
				}
			}
		}

		return AliasToSourceTagMapping.IsValid() ? AliasToSourceTagMapping->FindRef(InAlias) : NAME_None;
	}

private:
	/** Mapping from class name -> (alias -> source) */
	TMap<FTopLevelAssetPath, TSharedPtr<TMap<FName, FName>>> ClassToAliasTagsMapping;
};

/** Expression context to test the given asset data against the current text filter */
class FFrontendFilter_TextFilterExpressionContext_HackCopy : public ITextFilterExpressionContext
{
public:
	typedef TRemoveReference<const FAssetData&>::Type* FAssetFilterTypePtr;

	FFrontendFilter_TextFilterExpressionContext_HackCopy()
		: AssetPtr(nullptr)
		, bIncludeClassName(true)
		, bIncludeAssetPath(false)
		, bIncludeCollectionNames(true)
		, NameKeyName("Name")
		, PathKeyName("Path")
		, ClassKeyName("Class")
		, TypeKeyName("Type")
		, CollectionKeyName("Collection")
		, TagKeyName("Tag")
	{
	}

	void SetAsset(FAssetFilterTypePtr InAsset)
	{
		AssetPtr = InAsset;

		if (bIncludeAssetPath)
		{
			// Get the full asset path, and also split it so we can compare each part in the filter
			AssetPtr->PackageName.AppendString(AssetFullPath);
			AssetFullPath.ParseIntoArray(AssetSplitPath, TEXT("/"));
			AssetFullPath.ToUpperInline();

			if (bIncludeClassName)
			{
				// Get the full export text path as people sometimes search by copying this (requires class and asset path search to be enabled in order to match)
				AssetPtr->GetExportTextName(AssetExportTextName);
				AssetExportTextName.ToUpperInline();
			}
		}
	}

	void ClearAsset()
	{
		AssetPtr = nullptr;
		AssetFullPath.Reset();
		AssetExportTextName.Reset();
		AssetSplitPath.Reset();
		AssetCollectionNames.Reset();
	}

	void SetIncludeClassName(const bool InIncludeClassName)
	{
		bIncludeClassName = InIncludeClassName;
	}

	bool GetIncludeClassName() const
	{
		return bIncludeClassName;
	}

	void SetIncludeAssetPath(const bool InIncludeAssetPath)
	{
		bIncludeAssetPath = InIncludeAssetPath;
	}

	bool GetIncludeAssetPath() const
	{
		return bIncludeAssetPath;
	}

	void SetIncludeCollectionNames(const bool InIncludeCollectionNames)
	{
		bIncludeCollectionNames = InIncludeCollectionNames;
	}

	bool GetIncludeCollectionNames() const
	{
		return bIncludeCollectionNames;
	}

	virtual bool TestBasicStringExpression(const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode) const override
	{
		if (InValue.CompareName(AssetPtr->AssetName, InTextComparisonMode))
		{
			return true;
		}

		if (bIncludeAssetPath)
		{
			if (InValue.CompareFString(AssetFullPath, InTextComparisonMode))
			{
				return true;
			}

			for (const FString& AssetPathPart : AssetSplitPath)
			{
				if (InValue.CompareFString(AssetPathPart, InTextComparisonMode))
				{
					return true;
				}
			}
		}

		if (bIncludeClassName)
		{
			if (InValue.CompareFString(AssetPtr->AssetClassPath.ToString(), InTextComparisonMode))
			{
				return true;
			}
		}

		if (bIncludeClassName && bIncludeAssetPath)
		{
			// Only test this if we're searching the class name and asset path too, as the exported text contains the type and path in the string
			if (InValue.CompareFString(AssetExportTextName, InTextComparisonMode))
			{
				return true;
			}
		}

		if (bIncludeCollectionNames)
		{
			for (const FName& AssetCollectionName : AssetCollectionNames)
			{
				if (InValue.CompareName(AssetCollectionName, InTextComparisonMode))
				{
					return true;
				}
			}
		}

		return false;
	}

	virtual bool TestComplexExpression(const FName& InKey, const FTextFilterString& InValue, const ETextFilterComparisonOperation InComparisonOperation, const ETextFilterTextComparisonMode InTextComparisonMode) const override
	{
		// Special case for the asset name, as this isn't contained within the asset registry meta-data
		if (InKey == NameKeyName)
		{
			// Names can only work with Equal or NotEqual type tests
			if (InComparisonOperation != ETextFilterComparisonOperation::Equal && InComparisonOperation != ETextFilterComparisonOperation::NotEqual)
			{
				return false;
			}

			const bool bIsMatch = TextFilterUtils::TestBasicStringExpression(AssetPtr->AssetName, InValue, InTextComparisonMode);
			return (InComparisonOperation == ETextFilterComparisonOperation::Equal) ? bIsMatch : !bIsMatch;
		}

		// Special case for the asset path, as this isn't contained within the asset registry meta-data
		if (InKey == PathKeyName)
		{
			// Paths can only work with Equal or NotEqual type tests
			if (InComparisonOperation != ETextFilterComparisonOperation::Equal && InComparisonOperation != ETextFilterComparisonOperation::NotEqual)
			{
				return false;
			}

			// If the comparison mode is partial, then we only need to test the ObjectPath as that contains the other two as sub-strings
			bool bIsMatch = false;
			if (InTextComparisonMode == ETextFilterTextComparisonMode::Partial)
			{
				bIsMatch = TextFilterUtils::TestBasicStringExpression(AssetPtr->GetObjectPathString(), InValue, InTextComparisonMode);
			}
			else
			{
				bIsMatch = TextFilterUtils::TestBasicStringExpression(AssetPtr->GetObjectPathString(), InValue, InTextComparisonMode)
					|| TextFilterUtils::TestBasicStringExpression(AssetPtr->PackageName, InValue, InTextComparisonMode)
					|| TextFilterUtils::TestBasicStringExpression(AssetPtr->PackagePath, InValue, InTextComparisonMode);
			}
			return (InComparisonOperation == ETextFilterComparisonOperation::Equal) ? bIsMatch : !bIsMatch;
		}

		// Special case for the asset type, as this isn't contained within the asset registry meta-data
		if (InKey == ClassKeyName || InKey == TypeKeyName)
		{
			// Class names can only work with Equal or NotEqual type tests
			if (InComparisonOperation != ETextFilterComparisonOperation::Equal && InComparisonOperation != ETextFilterComparisonOperation::NotEqual)
			{
				return false;
			}

			const bool bIsMatch = TextFilterUtils::TestBasicStringExpression(AssetPtr->AssetClassPath.ToString(), InValue, InTextComparisonMode);
			return (InComparisonOperation == ETextFilterComparisonOperation::Equal) ? bIsMatch : !bIsMatch;
		}

		// Special case for collections, as these aren't contained within the asset registry meta-data
		if (InKey == CollectionKeyName || InKey == TagKeyName)
		{
			// Collections can only work with Equal or NotEqual type tests
			if (InComparisonOperation != ETextFilterComparisonOperation::Equal && InComparisonOperation != ETextFilterComparisonOperation::NotEqual)
			{
				return false;
			}

			bool bFoundMatch = false;
			for (const FName& AssetCollectionName : AssetCollectionNames)
			{
				if (TextFilterUtils::TestBasicStringExpression(AssetCollectionName, InValue, InTextComparisonMode))
				{
					bFoundMatch = true;
					break;
				}
			}

			return (InComparisonOperation == ETextFilterComparisonOperation::Equal) ? bFoundMatch : !bFoundMatch;
		}

		// Generic handling for anything in the asset meta-data
		{
			auto GetMetaDataValue = [this, &InKey](FString& OutMetaDataValue) -> bool
			{
				// Check for a literal key
				if (AssetPtr->GetTagValue(InKey, OutMetaDataValue))
				{
					return true;
				}

				// Check for an alias key
				const FName LiteralKey = FFrontendFilter_AssetPropertyTagAliases_HackCopy::Get().GetSourceTagFromAlias(*AssetPtr, InKey);
				if (!LiteralKey.IsNone() && AssetPtr->GetTagValue(LiteralKey, OutMetaDataValue))
				{
					return true;
				}

				return false;
			};

			FString MetaDataValue;
			if (GetMetaDataValue(MetaDataValue))
			{
				return TextFilterUtils::TestComplexExpression(MetaDataValue, InValue, InComparisonOperation, InTextComparisonMode);
			}
		}

		return false;
	}

private:
	/** Pointer to the asset we're currently filtering */
	FAssetFilterTypePtr AssetPtr;

	/** Full path of the current asset */
	FString AssetFullPath;

	/** The export text name of the current asset */
	FString AssetExportTextName;

	/** Split path of the current asset */
	TArray<FString> AssetSplitPath;

	/** Names of the collections that the current asset is in */
	TArray<FName> AssetCollectionNames;

	/** Are we supposed to include the class name in our basic string tests? */
	bool bIncludeClassName;

	/** Search inside the entire asset path? */
	bool bIncludeAssetPath;

	/** Search collection names? */
	bool bIncludeCollectionNames;

	/** Keys used by TestComplexExpression */
	const FName NameKeyName;
	const FName PathKeyName;
	const FName ClassKeyName;
	const FName TypeKeyName;
	const FName CollectionKeyName;
	const FName TagKeyName;
};

void FAssetRegistrySearchProvider::Search(FSearchQueryPtr SearchQuery)
{
	Async(EAsyncExecution::LargeThreadPool, [SearchQuery]() mutable {
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("FAssetRegistrySearchProvider Async Search");
		
		const IAssetRegistry& Registry = FAssetRegistryModule::GetRegistry();
			
		// Start by gathering all assets.
		TArray<FAssetData> Assets;
		Registry.GetAllAssets(Assets, true /* bIncludeOnlyOnDiskAssets */ );
		
		FTextFilterExpressionEvaluator TextFilterExpressionEvaluator(ETextFilterExpressionEvaluatorMode::Complex);
		TextFilterExpressionEvaluator.SetFilterText(FText::FromString(SearchQuery->QueryText));
		FFrontendFilter_TextFilterExpressionContext_HackCopy TextFilterExpressionContext;

		TArray<FSearchRecord> SearchResults;
		for (auto AssetIter = Assets.CreateIterator(); AssetIter; ++AssetIter)
		{
			const FAssetData& Asset = *AssetIter;
			TextFilterExpressionContext.SetAsset(&Asset);
			if (TextFilterExpressionEvaluator.TestTextFilter(TextFilterExpressionContext))
			{
				FSearchRecord Record;
				Record.AssetPath = Asset.GetObjectPathString();
				Record.AssetName = Asset.AssetName.ToString();
				Record.AssetClass = Asset.AssetClassPath;

				const float WorstCase = Record.AssetName.Len() + SearchQuery->QueryText.Len();
				Record.Score = -50.0f * (1.0f - (Algo::LevenshteinDistance(Record.AssetName.ToLower(), SearchQuery->QueryText.ToLower()) / WorstCase));

				SearchResults.Add(Record);
			}
			TextFilterExpressionContext.ClearAsset();
		}

		Async(EAsyncExecution::TaskGraphMainThread, [SearchQuery, SearchResults = MoveTemp(SearchResults)]() mutable {
			if (FSearchQuery::ResultsCallbackFunction ResultsCallback = SearchQuery->GetResultsCallback())
			{
				ResultsCallback(MoveTemp(SearchResults));
			}
		});
	});
}