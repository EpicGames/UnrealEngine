// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISearchProvider.h"

class FAssetRegistrySearchProvider : public ISearchProvider
{
public:
	virtual ~FAssetRegistrySearchProvider() { }
	virtual void Search(FSearchQueryPtr SearchQuery) override;
};
