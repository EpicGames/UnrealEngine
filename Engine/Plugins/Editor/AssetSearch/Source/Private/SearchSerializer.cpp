// Copyright Epic Games, Inc. All Rights Reserved.

#include "SearchSerializer.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "UObject/TextProperty.h"
#include "IAssetIndexer.h"

#define LOCTEXT_NAMESPACE "FSearchSerializer"

enum class ESearchIndexVersion
{
	Empty,
	Initial,
	IndexingInstancedSubObjects,
	
	// -----<new versions can be added above this line>-------------------------------------------------
	VersionPlusOne,
	LatestVersion = VersionPlusOne - 1
};

static FName StaticGetNativeClassName(const UClass* InClass)
{
	while (InClass && !InClass->HasAnyClassFlags(CLASS_Native))
	{
		InClass = InClass->GetSuperClass();
	}

	return InClass ? InClass->GetFName() : NAME_None;
}

FSearchSerializer::FSearchSerializer(const FAssetData& InAsset, FArchive& InAr)
	: JsonWriter(TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR> >::Create(&InAr))
{
	Initialize(InAsset);
}

FSearchSerializer::FSearchSerializer(const FAssetData& InAsset, FString* const Stream)
	: JsonWriter(TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR> >::Create(Stream))
{
	Initialize(InAsset);
}

void FSearchSerializer::Initialize(const FAssetData& InAsset)
{
	AssetData = InAsset;

	JsonWriter->WriteObjectStart();

	JsonWriter->WriteValue(TEXT("version"), GetVersion());

	//JsonWriter->WriteValue(TEXT("AssetName"), InAsset.AssetName.ToString());
	//JsonWriter->WriteValue(TEXT("AssetPath"), InAsset.ObjectPath.ToString());
	//JsonWriter->WriteValue(TEXT("AssetClass"), InAsset.AssetClassPath.ToString());

	JsonWriter->WriteObjectStart(TEXT("indexers"));
}

FSearchSerializer::~FSearchSerializer()
{
	JsonWriter->WriteObjectEnd();
	JsonWriter->WriteObjectEnd();
	JsonWriter->Close();
}

int32 FSearchSerializer::GetVersion()
{
	return (int32)ESearchIndexVersion::LatestVersion;
}

bool FSearchSerializer::IndexAsset(const UObject* InAsset, const TMap<FName, TUniquePtr<IAssetIndexer>>& Indexers)
{
	if (InAsset == nullptr)
	{
		return false;
	}

	bool bWasIndexed = false;

	UClass* IndexableClass = InAsset->GetClass();
	while (IndexableClass)
	{
		if (const TUniquePtr<IAssetIndexer>* IndexerPtr = Indexers.Find(IndexableClass->GetFName()))
		{
			IAssetIndexer* Indexer = IndexerPtr->Get();

			bWasIndexed = true;
			BeginIndexer(Indexer);
			Indexer->IndexAsset(InAsset, *this);
			EndIndexer();
		}

		IndexableClass = IndexableClass->GetSuperClass();
	}

	// 
	TArray<UObject*> TempNestedAssets = NestedAssets;
	NestedAssets.Reset();
	for (UObject* NestedAsset : TempNestedAssets)
	{
		IndexAsset(NestedAsset, Indexers);
	}

	return true;
}

void FSearchSerializer::BeginIndexer(const IAssetIndexer* InIndexer)
{
	check(CurrentIndexer == nullptr);
	CurrentIndexer = InIndexer;

	JsonWriter->WriteObjectStart(InIndexer->GetName());
	JsonWriter->WriteValue(TEXT("version"), InIndexer->GetVersion());

	JsonWriter->WriteArrayStart(TEXT("objects"));
}

void FSearchSerializer::EndIndexer()
{
	JsonWriter->WriteArrayEnd();
	JsonWriter->WriteObjectEnd();

	CurrentIndexer = nullptr;
}

void FSearchSerializer::BeginIndexingObject(const UObject* InObject, const FText& InFriendlyName)
{
	BeginIndexingObject(InObject, *FTextInspector::GetSourceString(InFriendlyName));
}

void FSearchSerializer::BeginIndexingObject(const UObject* InObject, const FString& InFriendlyName)
{
	ensure(CurrentObject == nullptr);
	ensure(InObject->GetOutermost()->GetFName() == AssetData.PackageName);

	CurrentObject = InObject;
	CurrentObjectName = InFriendlyName;
	CurrentObjectClass = StaticGetNativeClassName(InObject->GetClass()).ToString();
	CurrentObjectPath = InObject->GetPathName();
}

void FSearchSerializer::IndexNestedAsset(UObject* InNestedAsset)
{
	NestedAssets.Add(InNestedAsset);
}

void FSearchSerializer::EndIndexingObject()
{
	if (Values.Num() > 0)
	{
		JsonWriter->WriteObjectStart();
		JsonWriter->WriteValue(TEXT("name"), CurrentObjectName);
		JsonWriter->WriteValue(TEXT("path"), CurrentObjectPath);
		JsonWriter->WriteValue(TEXT("native_class"), CurrentObjectClass);

		JsonWriter->WriteArrayStart(TEXT("properties"));

		for (const FIndexedValue& IndexedValue : Values)
		{
			JsonWriter->WriteObjectStart();

			JsonWriter->WriteValue(TEXT("name"), IndexedValue.PropertyName);
			JsonWriter->WriteValue(TEXT("field"), IndexedValue.PropertyFieldClass);

			if (IndexedValue.PropertyClass != NAME_None)
			{
				JsonWriter->WriteValue(TEXT("class"), IndexedValue.PropertyClass.ToString());
			}

			if (!IndexedValue.Text.IsEmpty())
			{
				JsonWriter->WriteValue(TEXT("value_text"), IndexedValue.Text);
			}

			if (!IndexedValue.HiddenText.IsEmpty())
			{
				JsonWriter->WriteValue(TEXT("value_hidden"), IndexedValue.HiddenText);
			}

			JsonWriter->WriteObjectEnd();
		}

		JsonWriter->WriteArrayEnd();

		JsonWriter->WriteObjectEnd();
	}

	Values.Reset();

	CurrentObject = nullptr;
	CurrentObjectClass.Reset();
	CurrentObjectPath.Reset();
}

void FSearchSerializer::IndexProperty(const FString& InName, const FText& InValue)
{
	IndexProperty(*FTextProperty::StaticClass(), nullptr, InName, *FTextInspector::GetSourceString(InValue));
}

void FSearchSerializer::IndexProperty(const FString& InName, const FName& InValue)
{
	IndexProperty(*FNameProperty::StaticClass(), nullptr, InName, InValue.ToString());
}

void FSearchSerializer::IndexProperty(const FString& InName, const FString& InValue)
{
	IndexProperty(*FStrProperty::StaticClass(), nullptr, InName, InValue);
}

void FSearchSerializer::IndexProperty(const UClass* InPropertyClass, const FString& InName, const FString& InValue)
{
	IndexProperty(*FObjectProperty::StaticClass(), InPropertyClass, InName, InValue);
}

void FSearchSerializer::IndexProperty(const UClass* InPropertyClass, const FString& InName, const FText& InValue)
{
	IndexProperty(*FObjectProperty::StaticClass(), InPropertyClass, InName, *FTextInspector::GetSourceString(InValue));
}

void FSearchSerializer::IndexProperty(const FProperty* InProperty, const FString& InValue)
{
	const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(InProperty);
	IndexProperty(*InProperty->GetClass(), ObjectProperty ? ObjectProperty->PropertyClass : nullptr, InProperty->GetDisplayNameText().ToString(), InValue);
}

void FSearchSerializer::IndexProperty(const FFieldClass& InPropertyFieldClass, const UClass* InPropertyClass, const FString& InName, const FString& InValue)
{
	if (ensure(CurrentObject))
	{
		// Don't index empty stuff.
		if (InValue.IsEmpty())
		{
			return;
		}

		FIndexedValue& IndexedValue = Values.Emplace_GetRef();
		IndexedValue.PropertyName = InName;
		IndexedValue.PropertyFieldClass = InPropertyFieldClass.GetName();
		IndexedValue.PropertyClass = StaticGetNativeClassName(InPropertyClass);
		IndexedValue.Text = InValue;
	}

	///** Gets the list of all field classes in existence */
	//static TArray<FFieldClass*>& GetAllFieldClasses();
	///** Gets a mapping of all field class names to the actually class objects */
	//static TMap<FName, FFieldClass*>& GetNameToFieldClassMap();
}

#undef LOCTEXT_NAMESPACE
