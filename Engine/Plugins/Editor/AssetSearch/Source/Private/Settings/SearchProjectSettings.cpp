// Copyright Epic Games, Inc. All Rights Reserved.

#include "SearchProjectSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SearchProjectSettings)

USearchProjectSettings::USearchProjectSettings()
{
}

