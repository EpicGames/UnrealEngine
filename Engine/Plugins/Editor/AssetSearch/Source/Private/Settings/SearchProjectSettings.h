// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "SearchProjectSettings.generated.h"

struct FDirectoryPath;

UENUM()
enum class ESearchIntermediateStorage : uint8
{
	// Stores the json snippets of indexable data in the DDC so that they can be downloaded as needed.
	DerivedDataCache,
	// Stores the json snippets and the json hash in the tag data for the asset.  This increases the size of all 
	// assets that are indexed, but can be a great benefit to teams that don't have a large shared DDC.
	AssetTagData
};

UCLASS(config = Editor, defaultconfig, meta=(DisplayName="Search"))
class USearchProjectSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	USearchProjectSettings();

	UPROPERTY(config, EditAnywhere, Category=General, Meta = (ConfigRestartRequired = true))
	ESearchIntermediateStorage IntermediateStorage = ESearchIntermediateStorage::DerivedDataCache;

	UPROPERTY(config, EditAnywhere, Category=General)
	TArray<FDirectoryPath> IgnoredPaths;
	
	/** Disable put/fetch operations with the DDC, without changing storage location */
	UPROPERTY(config, EditAnywhere, Category=General)
	bool bDisableDDC;
};
