// Copyright Epic Games, Inc. All Rights Reserved.

#include "SearchUserSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SearchUserSettings)

USearchUserSettings::USearchUserSettings()
{
	DefaultOptions.ParallelDownloads = 100;
	DefaultOptions.DownloadProcessRate = 30;
	DefaultOptions.AssetScanRate = 1000;

	BackgroundtOptions.ParallelDownloads = 1;
	BackgroundtOptions.DownloadProcessRate = 30;
	BackgroundtOptions.AssetScanRate = 200;
}

const FSearchPerformance& USearchUserSettings::GetPerformanceOptions() const
{
	return (SearchInForeground > 0) ? DefaultOptions : BackgroundtOptions;
}
