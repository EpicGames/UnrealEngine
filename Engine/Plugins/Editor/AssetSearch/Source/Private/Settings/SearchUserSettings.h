// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "SearchUserSettings.generated.h"

struct FDirectoryPath;

USTRUCT()
struct FSearchPerformance
{
	GENERATED_BODY();

public:
	UPROPERTY(EditAnywhere, Category = "Performance")
	int32 ParallelDownloads = 0;

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Performance")
	int32 DownloadProcessRate = 0;

	UPROPERTY(EditAnywhere, AdvancedDisplay, Category = "Performance")
	int32 AssetScanRate = 0;
};

UCLASS(config = EditorPerProjectUserSettings, meta=(DisplayName="Search"))
class USearchUserSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	USearchUserSettings();

	/** Enable this to begin using search. */
	UPROPERTY(config, EditAnywhere, Category = General)
	bool bEnableSearch;
	
	/** Enable sqlite integrity checks on database load/setup. Slow. */
	UPROPERTY(config, EditAnywhere, Category=General)
	bool bEnableIntegrityChecks;

	UPROPERTY(config, EditAnywhere, Category=General)
	TArray<FDirectoryPath> IgnoredPaths;

	UPROPERTY(config, EditAnywhere, Category=General)
	bool bShowAssetsNeedingIndexing = true;

	UPROPERTY(config, EditAnywhere, Category = General)
	bool bAutoExpandAssets = true;

	UPROPERTY(config, EditAnywhere, Category = Performance)
	bool bThrottleInBackground = true;

	UPROPERTY(config, EditAnywhere, Category = Performance)
	FSearchPerformance DefaultOptions;

	UPROPERTY(config, EditAnywhere, Category = Performance)
	FSearchPerformance BackgroundtOptions;

	const FSearchPerformance& GetPerformanceOptions() const;

public:
	UPROPERTY(Transient)
	int32 SearchInForeground = 0;
};
