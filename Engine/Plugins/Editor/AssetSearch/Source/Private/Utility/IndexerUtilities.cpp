// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utility/IndexerUtilities.h"
#include "GameplayTagContainer.h"
#include "UObject/EnumProperty.h"
#include "UObject/TextProperty.h"

static bool IsPropertyIndexable(const TPropertyValueIterator<FProperty>& It, const FProperty* Property)
{
	// Don't index transient properties.
	if (Property->HasAnyPropertyFlags(CPF_Transient))
	{
		return false;
	}

	// Don't index anything we don't expose to the editor.
	if (!Property->HasAnyPropertyFlags(CPF_Edit | CPF_BlueprintVisible | CPF_AssetRegistrySearchable))
	{
		TArray<const FProperty*> PropertyChain;
		It.GetPropertyChain(PropertyChain);

		if (PropertyChain.Num() > 0)
		{
			// If this is a property in a container property (that's not a struct) then that array is indexable, so this should be too.
			if (CastField<FArrayProperty>(PropertyChain.Last()) || CastField<FSetProperty>(PropertyChain.Last()) || CastField<FMapProperty>(PropertyChain.Last()))
			{
				return true;
			}
		}
		
		return false;
	}

	return true;
}

void FIndexerUtilities::IterateIndexableProperties(const UObject* InObject, TFunctionRef<void(const FProperty* /*Property*/, const FString& /*Value*/)> Callback)
{
	IterateIndexableProperties(InObject->GetClass(), InObject, Callback);
}

void FIndexerUtilities::IterateIndexableProperties(const UStruct* InStruct, const void* InStructValue, TFunctionRef<void(const FProperty* /*Property*/, const FString& /*Value*/)> Callback)
{
	for (TPropertyValueIterator<FProperty> It(InStruct, InStructValue, EPropertyValueIteratorFlags::FullRecursion, EFieldIteratorFlags::ExcludeDeprecated); It; ++It)
	{
		const FProperty* Property = It.Key();

		// Don't index a property we wouldn't normally expose to the user.
		if (!IsPropertyIndexable(It, Property))
		{
			It.SkipRecursiveProperty();
			continue;
		}

		void const* ValuePtr = It.Value();

		FString Text;
		if (const FNameProperty* NameProperty = CastField<FNameProperty>(Property))
		{
			const FName Value = NameProperty->GetPropertyValue(ValuePtr);
			if (!Value.IsNone())
			{
				Text = Value.ToString();
			}
		}
		else if (const FStrProperty* StringProperty = CastField<FStrProperty>(Property))
		{
			Text = StringProperty->GetPropertyValue(ValuePtr);
		}
		else if (const FTextProperty* TextProperty = CastField<FTextProperty>(Property))
		{
			const FText Value = TextProperty->GetPropertyValue(ValuePtr);
			Text = *FTextInspector::GetSourceString(Value);
		}
		else if (const FByteProperty* ByteProperty = CastField<FByteProperty>(Property))
		{
			if (const UEnum* Enum = ByteProperty->Enum)
			{
				const int64 Value = ByteProperty->GetSignedIntPropertyValue(ValuePtr);
				FText DisplayName = Enum->GetDisplayNameTextByValue(Value);
				Text = *FTextInspector::GetSourceString(DisplayName);	
			}
		}
		else if (const FEnumProperty* EnumProperty = CastField<FEnumProperty>(Property))
		{
			const int64 Value = EnumProperty->GetUnderlyingProperty()->GetSignedIntPropertyValue(ValuePtr);
			FText DisplayName = EnumProperty->GetEnum()->GetDisplayNameTextByValue(Value);
			Text = *FTextInspector::GetSourceString(DisplayName);
		}
		else if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
		{
			if (const UObject* Object = ObjectProperty->GetPropertyValue(ValuePtr))
			{
				if (Object->HasAnyFlags(RF_Public) && Object->IsAsset())
				{
					Text = Object->GetName();
				}
				else if (Object->HasAnyFlags(RF_Transient))
				{
					// Don't do anything with transient objects.
					continue;
				}
				// If the property is "Instanced" then we may need to iterate through the object, only do this for
				// edit inline new classes, we don't care about inner-reference objects that already already tracked
				// in some other way, that's up to the caller of this function to handle, we just want to handle
				// iterating the obviously owned data.
				else if (Property->HasAllPropertyFlags(CPF_ExportObject) && Object->GetClass()->HasAllClassFlags(CLASS_EditInlineNew))
				{
					// Add the inner properties of this instanced object.
					IterateIndexableProperties(Object, Callback);
					continue;
				}
			}
		}
		else if (const FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(Property))
		{
			FSoftObjectPtr SoftObject = SoftObjectProperty->GetPropertyValue(ValuePtr);
			if (!SoftObject.IsNull())
			{
				const FSoftObjectPath& SoftObjectPath = SoftObject.ToSoftObjectPath();
				Text = SoftObject.GetAssetName();

				// If Soft Object Path is reference to AActor instance in the world, GetSubPathString() returns this actor instance path
				// GetAssetName() would just return name of the level name, which won't provide valuable info for the search
				if (!SoftObjectPath.GetSubPathString().IsEmpty())
				{
					Text.Append(TEXT(".")).Append(SoftObjectPath.GetSubPathString());
				}	
			}
		}
		else if (const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
		{
			if (StructProperty->Struct == FGameplayTag::StaticStruct())
			{
				const FGameplayTag* GameplayTag = static_cast<const FGameplayTag*>(ValuePtr);
				Text = GameplayTag->GetTagName().ToString();
				
				It.SkipRecursiveProperty();
			}
			else if (StructProperty->Struct == FGameplayTagContainer::StaticStruct())
			{
				const FGameplayTagContainer* GameplayTagContainer = static_cast<const FGameplayTagContainer*>(ValuePtr);
				for (auto TagIter = GameplayTagContainer->CreateConstIterator(); TagIter; ++TagIter)
				{
					Text = TagIter->GetTagName().ToString();
					Callback(Property, Text);
				}

				It.SkipRecursiveProperty();
				continue;
			}
			else if (UScriptStruct* ScriptStruct = StructProperty->Struct.Get())
			{
				// We don't want to record all ScriptStructs, only those explicitly flagged.
				TArray<const FProperty*> PropertyChain;
				It.GetPropertyChain(PropertyChain);
				const bool bIsSearchable = PropertyChain.ContainsByPredicate([](const FProperty* Property) { return Property->HasAnyPropertyFlags(CPF_AssetRegistrySearchable); });
				if (bIsSearchable)
				{
					ScriptStruct->ExportText(Text, ValuePtr, nullptr, nullptr, PPF_None, nullptr);
				}
			}
			//else if (StructProperty->Struct == FGuid::StaticStruct())
			//{
			//	const FGuid* Guid = static_cast<const FGuid*>(ValuePtr);
			//	Text = Guid->ToString();
			//}
		}

		// Ignore empty records
		if (!Text.IsEmpty())
		{
			Callback(Property, Text);
		}
	}
}
