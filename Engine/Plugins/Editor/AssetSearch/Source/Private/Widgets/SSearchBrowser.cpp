// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSearchBrowser.h"
#include "Framework/Views/TableViewMetadata.h"
#include "SSearchTreeRow.h"

#include "Framework/MultiBox/MultiBoxBuilder.h"

// Common classes for the picker
#include "Widgets/Input/SComboButton.h"
#include "SearchModel.h"
#include "IAssetSearchModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Input/SHyperlink.h"
#include "Widgets/Input/SSearchBox.h"
#include "Settings/SearchUserSettings.h"
#include "SearchStyle.h"
#include "Widgets/SToolTip.h"
#include "ThumbnailRendering/ThumbnailManager.h"

#define LOCTEXT_NAMESPACE "SObjectBrowser"

DEFINE_LOG_CATEGORY_STATIC(LogObjectBrowser, Log, All)
//;
//TODO Expose TSharedRef<SWidget> SAssetViewItem::CreateToolTipWidget() const via IContentBrowserSingleton.

SSearchBrowser::~SSearchBrowser()
{
	if (UObjectInitialized())
	{
		GetMutableDefault<USearchUserSettings>()->SearchInForeground--;
	}
}

void SSearchBrowser::Construct( const FArguments& InArgs )
{
	USearchUserSettings* UserSettings = GetMutableDefault<USearchUserSettings>();

	if (!UserSettings->bEnableSearch)
	{
		UserSettings->bEnableSearch = true;
		UserSettings->SaveConfig();
	}

	UserSettings->SearchInForeground++;

	SortByColumn = SSearchTreeRow::NAME_ColumnName;
	SortMode = EColumnSortMode::Ascending;


	AssetRegistry = &FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();

	ChildSlot
	[
		SNew(SBorder)
		.Padding(FMargin(3))
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		[
			SNew(SVerticalBox)

			+ SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					SNew(SSearchBox)
					.HintText(LOCTEXT("SearchHint", "Search"))
					.OnTextCommitted(this, &SSearchBrowser::OnSearchTextCommited)
					.OnTextChanged(this, &SSearchBrowser::OnSearchTextChanged)
					.IsSearching(this, &SSearchBrowser::IsSearching)
					.DelayChangeNotificationsWhileTyping(true)
				]
			]

			+ SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				.Padding(FMargin(0.0f, 4.0f))
				[
					SNew(SOverlay)

					+ SOverlay::Slot()
					[
						SAssignNew(SearchTreeView, STreeView< TSharedPtr<FSearchNode> >)
						.TreeItemsSource(&SearchResults)
						.SelectionMode(ESelectionMode::Single)
						.OnGenerateRow(this, &SSearchBrowser::HandleListGenerateRow)
						.OnGetChildren(this, &SSearchBrowser::GetChildrenForInfo)
						.OnSelectionChanged(this, &SSearchBrowser::HandleListSelectionChanged)
						.HeaderRow
						(
							SAssignNew(HeaderColumns, SHeaderRow)

							+ SHeaderRow::Column(SSearchTreeRow::NAME_ColumnName)
							.DefaultLabel(LOCTEXT("ColumnName", "Name"))
							.FillWidth(70)
						)
					]

					+ SOverlay::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Visibility(EVisibility::HitTestInvisible)
						.Clipping(EWidgetClipping::Inherit)
						.Text(this, &SSearchBrowser::GetSearchBackgroundText)
						.Justification(ETextJustify::Center)
						.Font(FCoreStyle::GetDefaultFontStyle("Regular", 30))
						.ColorAndOpacity(FLinearColor(1,1,1,0.05f))
						.RenderTransformPivot(FVector2D(0.5, 0.5))
						.RenderTransform(FSlateRenderTransform(FQuat2D(FMath::DegreesToRadians(-30.0f))))
					]
				]
			]
			
			+ SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0, 0, 0, 1)
			[
				SNew(SHorizontalBox)

				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(8, 0)
				[
					SNew(SImage)
					.Image(FSearchStyle::Get().GetBrush("Stats"))
					.ToolTip(
						SNew(SToolTip)
						[
							SNew(SVerticalBox)

							+ SVerticalBox::Slot()
							.AutoHeight()
							[
								SNew(STextBlock)
								.Text(this, &SSearchBrowser::GetAdvancedStatus)
							]
						]
					)
				]

				// Asset Stats 
				+ SHorizontalBox::Slot()
				.FillWidth(1.f)
				.VAlign(VAlign_Center)
				.Padding(2, 0)
				[
					SNew(STextBlock)
					.Text(this, &SSearchBrowser::GetStatusText)
				]

				// Index unindexed items
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SHyperlink)
					.Text(this, &SSearchBrowser::GetUnindexedAssetsText)
					.ToolTipText(LOCTEXT("AssetsNeedIndexingTooltip", "Click this to open and index the assets that are don't have any index data or their index data was found to be out of date."))
					.Visibility_Lambda([] { return GetDefault<USearchUserSettings>()->bShowAssetsNeedingIndexing ? EVisibility::Visible : EVisibility::Collapsed; })
					.OnNavigate(this, &SSearchBrowser::HandleForceIndexOfAssetsMissingIndex)
				]

				// View Options Button
				+ SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew( SComboButton )
					.ContentPadding(0)
					.ForegroundColor( FSlateColor::UseForeground() )
					.ButtonStyle( FAppStyle::Get(), "ToggleButton" )
					.OnGetMenuContent(this, &SSearchBrowser::GetViewMenuWidget)
					.ButtonContent()
					[
						SNew(SImage)
						.Image( FAppStyle::GetBrush("GenericViewButton") )
					]
				]
			]
		]
	];

	RefreshList();
}

void SSearchBrowser::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	const int32 MinSizeForTypeColumn = 400;
	if (HeaderColumns->GetColumns().Num() == 1 && AllottedGeometry.GetLocalSize().X > MinSizeForTypeColumn)
	{
		HeaderColumns->AddColumn(
			SHeaderRow::Column(SSearchTreeRow::NAME_ColumnType)
			.DefaultLabel(LOCTEXT("ColumnType", "Type"))
			.FillWidth(30)
		);
	}
	else if (HeaderColumns->GetColumns().Num() > 1 && AllottedGeometry.GetLocalSize().X < MinSizeForTypeColumn)
	{
		HeaderColumns->RemoveColumn(SSearchTreeRow::NAME_ColumnType);
	}
}

TSharedRef<SWidget> SSearchBrowser::GetViewMenuWidget()
{
	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.BeginSection("Results", LOCTEXT("ShowHeading", "Show"));
	{
		MenuBuilder.AddMenuEntry(
			LOCTEXT("ToggleAutoExpandAssets", "Auto Expand Assets"),
			LOCTEXT("ToggleAutoExpandAssetsToolTip", "When enabled, we automatically expand the assets in the results."),
			FSlateIcon(),
			FUIAction(
				FExecuteAction::CreateLambda([] { GetMutableDefault<USearchUserSettings>()->bAutoExpandAssets = !GetDefault<USearchUserSettings>()->bAutoExpandAssets; }),
				FCanExecuteAction(),
				FIsActionChecked::CreateLambda([] { return GetDefault<USearchUserSettings>()->bAutoExpandAssets; })
			),
			NAME_None,
			EUserInterfaceActionType::ToggleButton
		);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

FText SSearchBrowser::GetSearchBackgroundText() const
{
	if (FilterString.Len() > 0 && !IsSearching() && SearchResults.Num() == 0)
	{
		return LOCTEXT("FoundNoResults", "¯\\_(ツ)_/¯");
	}
	else if (FilterString.Len() == 0)
	{
		const IAssetSearchModule& SearchModule = IAssetSearchModule::Get();
		const FSearchStats SearchStats = SearchModule.GetStats();

		if (SearchStats.IsUpdating() && SearchStats.TotalRecords > 0)
		{
			return FText::Format(LOCTEXT("SearchNumberOfThings", "Search\n{0} Things!"), SearchStats.TotalRecords);
		}
		else
		{
			return LOCTEXT("SearchAllTheThings", "Search\nAll The Things!");
		}
	}
	else
	{
		return FText::GetEmpty();
	}
}

FText SSearchBrowser::GetStatusText() const
{
	const IAssetSearchModule& SearchModule = IAssetSearchModule::Get();
	const FSearchStats SearchStats = SearchModule.GetStats();
	
	if (SearchStats.IsUpdating())
	{
		return LOCTEXT("Updating", "Updating...  (You can search any time)");
	}
	else
	{
		return LOCTEXT("Ready", "Ready");
	}
}

FText SSearchBrowser::GetAdvancedStatus() const
{
	const IAssetSearchModule& SearchModule = IAssetSearchModule::Get();
	const FSearchStats SearchStats = SearchModule.GetStats();
	return FText::Format(LOCTEXT("AdvancedSearchStatusTextFmt", "Scanning {0}\nProcessing {1}\nUpdating {2}\n\nTotal Records {3}"), SearchStats.Scanning, SearchStats.Processing, SearchStats.Updating, SearchStats.TotalRecords);
}

FText SSearchBrowser::GetUnindexedAssetsText() const
{
	const IAssetSearchModule& SearchModule = IAssetSearchModule::Get();
	const FSearchStats SearchStats = SearchModule.GetStats();

	if (SearchStats.AssetsMissingIndex > 0)
	{
		return FText::Format(LOCTEXT("UnindexedAssetsLinkFormat", "{0} Need Indexing"), SearchStats.AssetsMissingIndex);
	}

	return FText::GetEmpty();
}

void SSearchBrowser::HandleForceIndexOfAssetsMissingIndex()
{
	IAssetSearchModule& SearchModule = IAssetSearchModule::Get();
	SearchModule.ForceIndexOnAssetsMissingIndex();
}

FReply SSearchBrowser::OnRefresh()
{
	RefreshList();

	return FReply::Handled();
}

EColumnSortMode::Type SSearchBrowser::GetColumnSortMode(const FName ColumnId) const
{
	if (SortByColumn == ColumnId)
	{
		return SortMode;
	}

	return EColumnSortMode::None;
}

void SSearchBrowser::OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode)
{
	SortByColumn = ColumnId;
	SortMode = InSortMode;

	RefreshList();
}

void SSearchBrowser::RefreshList()
{
	SearchResults.Reset();
	SearchResultHierarchy.Reset();

	SearchTreeView->RequestListRefresh();

	if (!FilterText.IsEmpty())
	{
		{
			FSearchQueryPtr ActiveSearch = ActiveSearchPtr.Pin();
			if (ActiveSearch)
			{
				ActiveSearch->ClearResultsCallback();
				ActiveSearchPtr.Reset();
			}
		}

		const bool bAutoExpandAssets = GetDefault<USearchUserSettings>()->bAutoExpandAssets;

		TWeakPtr<SSearchBrowser> WeakSelf = SharedThis(this);

		FSearchQueryPtr NewQuery = MakeShared<FSearchQuery, ESPMode::ThreadSafe>(FilterText.ToString());
		NewQuery->SetResultsCallback([this, WeakSelf, bAutoExpandAssets](TArray<FSearchRecord>&& InResults) {
			check(IsInGameThread());

			if (WeakSelf.IsValid())
			{
				for (int32 ResultIndex = 0; ResultIndex < InResults.Num(); ResultIndex++)
				{
					AppendResult(MoveTemp(InResults[ResultIndex]));
				}

				SearchResults.Reset();
				for (auto& Entry : SearchResultHierarchy)
				{
					SearchResults.Add(Entry.Value);

					if (bAutoExpandAssets)
					{
						SearchTreeView->SetItemExpansion(Entry.Value, true);
					}
				}

				SearchResults.Sort([](const TSharedPtr<FSearchNode>& A, const TSharedPtr<FSearchNode>& B) {
					return A->GetMaxScore() < B->GetMaxScore();
				});

				SearchTreeView->RequestListRefresh();
			}
		});

		IAssetSearchModule& SearchModule = IAssetSearchModule::Get();
		SearchModule.Search(NewQuery);

		ActiveSearchPtr = NewQuery;
	}
}

void SSearchBrowser::AppendResult(FSearchRecord&& InResult)
{
	TSharedPtr<FAssetNode> ExistingAssetNode = SearchResultHierarchy.FindRef(InResult.AssetPath);
	if (!ExistingAssetNode.IsValid())
	{
		ExistingAssetNode = MakeShared<FAssetNode>(InResult);
		SearchResultHierarchy.Add(InResult.AssetPath, ExistingAssetNode);
	}
	else
	{
		ExistingAssetNode->Append(InResult);
	}
}

void SSearchBrowser::OnSearchTextCommited(const FText& InText, ETextCommit::Type InCommitType)
{
	TryRefreshingSearch(InText);
}

void SSearchBrowser::OnSearchTextChanged(const FText& InText)
{
	const int32 Length = InText.ToString().Len();

	if (Length > 3)
	{
		TryRefreshingSearch(InText);
	}
	else if (Length == 0)
	{
		TryRefreshingSearch(InText);
	}
}

void SSearchBrowser::TryRefreshingSearch(const FText& InText)
{
	if (FilterText.ToString() != InText.ToString())
	{
		FilterText = InText;
		FilterString = FilterText.ToString();

		RefreshList();
	}
}

TSharedRef<ITableRow> SSearchBrowser::HandleListGenerateRow(TSharedPtr<FSearchNode> ObjectPtr, const TSharedRef<STableViewBase>& OwnerTable)
{
	return SNew(SSearchTreeRow, OwnerTable, AssetRegistry, UThumbnailManager::Get().GetSharedThumbnailPool())
		.Object(ObjectPtr)
		.HighlightText(FilterText);
}

void SSearchBrowser::GetChildrenForInfo(TSharedPtr<FSearchNode> InNode, TArray< TSharedPtr<FSearchNode> >& OutChildren)
{
	InNode->GetChildren(OutChildren);
}

void SSearchBrowser::HandleListSelectionChanged(TSharedPtr<FSearchNode> InNode, ESelectInfo::Type SelectInfo)
{
}

bool SSearchBrowser::IsSearching() const
{
	return ActiveSearchPtr.IsValid();
}

#undef LOCTEXT_NAMESPACE
