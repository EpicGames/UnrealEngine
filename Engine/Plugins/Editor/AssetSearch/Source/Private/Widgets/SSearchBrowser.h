// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SearchQuery.h"
#include "Widgets/Views/SHeaderRow.h"

class STableViewBase;
template <typename ItemType> class STreeView;

class IDetailsView;
class ITableRow;
class FSearchNode;
class FAssetNode;
class FMenuBuilder;
class IAssetRegistry;
class SHeaderRow;

/**
 * Implements the undo history panel.
 */
class SSearchBrowser : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SSearchBrowser) { }
	SLATE_END_ARGS()

public:

	virtual ~SSearchBrowser();

	/**
	 * Construct this widget
	 *
	 * @param InArgs The declaration data for this widget.
	 */
	void Construct( const FArguments& InArgs );

public:
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

private:

	TSharedRef<SWidget> GetViewMenuWidget();

	FText GetSearchBackgroundText() const;
	FText GetStatusText() const;
	FText GetAdvancedStatus() const;
	FText GetUnindexedAssetsText() const;

	void HandleForceIndexOfAssetsMissingIndex();

	FReply OnRefresh();

	EColumnSortMode::Type GetColumnSortMode(const FName ColumnId) const;
	void OnColumnSortModeChanged(const EColumnSortPriority::Type SortPriority, const FName& ColumnId, const EColumnSortMode::Type InSortMode);

	void RefreshList();
	void AppendResult(FSearchRecord&& InResult);

	void OnSearchTextCommited(const FText& InText, ETextCommit::Type InCommitType);
	void OnSearchTextChanged(const FText& InText);
	void TryRefreshingSearch(const FText& InText);

	TSharedRef<ITableRow> HandleListGenerateRow(TSharedPtr<FSearchNode> TransactionInfo, const TSharedRef<STableViewBase>& OwnerTable);
	void GetChildrenForInfo(TSharedPtr<FSearchNode> InInfo, TArray< TSharedPtr<FSearchNode> >& OutChildren);

	void HandleListSelectionChanged(TSharedPtr<FSearchNode> TransactionInfo, ESelectInfo::Type SelectInfo);

	bool IsSearching() const;

private:

	FText FilterText;

	// Filters
	FString FilterString;

	FSearchQueryWeakPtr ActiveSearchPtr;
	
	TMap<FString, TSharedPtr<FAssetNode>> SearchResultHierarchy;
	TArray< TSharedPtr<FSearchNode> > SearchResults;

	TSharedPtr< STreeView< TSharedPtr<FSearchNode> > > SearchTreeView;

	IAssetRegistry* AssetRegistry = nullptr;

	TSharedPtr<SHeaderRow> HeaderColumns;

	FName SortByColumn;
	EColumnSortMode::Type SortMode;
};
