// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSearchTreeRow.h"
#include "Engine/Blueprint.h"
#include "SearchModel.h"
#include "IAssetTools.h"
#include "Styling/SlateIconFinder.h"
#include "AssetToolsModule.h"
#include "IAssetTypeActions.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "IMaterialEditor.h"

#include "Materials/Material.h"
#include "Materials/MaterialFunction.h"
#include "Misc/Paths.h"

#define LOCTEXT_NAMESPACE "SObjectBrowserTableRow"

FName SSearchTreeRow::NAME_ColumnName(TEXT("Name"));
FName SSearchTreeRow::NAME_ColumnType(TEXT("Type"));
//FName SSearchTreeRow::CategoryProperty(TEXT("Property"));
//FName SSearchTreeRow::CategoryPropertyValue(TEXT("PropertyValue"));

void SSearchTreeRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, IAssetRegistry* InAssetRegistry, TSharedPtr<FAssetThumbnailPool> InThumbnailPool)
{
	BrowserObject = InArgs._Object;
	AssetRegistry = InAssetRegistry;
	ThumbnailPool = InThumbnailPool;

	FSuperRowType::Construct(
		FSuperRowType::FArguments(),
		InOwnerTableView
	);
}

TSharedRef<SWidget> SSearchTreeRow::GenerateWidget()
{
	return SNew(STextBlock)
		.Text(FText::FromString(BrowserObject->GetText()));
}

TSharedRef<SWidget> SSearchTreeRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	TSharedRef<SHorizontalBox> HorizBox = SNew(SHorizontalBox);

	FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");

	if (ColumnName == NAME_ColumnName)
	{
		TSharedPtr<SWidget> IconWidget;
		if (BrowserObject->GetType() == ESearchNodeType::Object)
		{
			TSharedPtr<FAssetObjectNode> ObjectNode = StaticCastSharedPtr<FAssetObjectNode>(BrowserObject);

			if (UClass* ObjectClass = UClass::TryFindTypeSlow<UClass>(*ObjectNode->object_native_class, EFindFirstObjectOptions::ExactClass))
			{
				FSlateIcon ClassIcon = FSlateIconFinder::FindIconForClass(ObjectClass);
				if (ClassIcon.IsSet())
				{
					IconWidget = SNew(SImage)
						.Image(ClassIcon.GetIcon());
				}
			}
		}
		else if (BrowserObject->GetType() == ESearchNodeType::Asset)
		{
			TSharedPtr<FAssetNode> ObjectNode = StaticCastSharedPtr<FAssetNode>(BrowserObject);

			if (UClass* ObjectClass = UClass::TryFindTypeSlow<UClass>(ObjectNode->AssetClass, EFindFirstObjectOptions::ExactClass))
			{
				TSharedPtr<IAssetTypeActions> AssetActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(ObjectClass).Pin();

				FSlateIcon ClassIcon = FSlateIconFinder::FindIconForClass(ObjectClass);
				if (ClassIcon.IsSet())
				{
					IconWidget = SNew(SImage)
						.Image(ClassIcon.GetIcon())
						.ColorAndOpacity(AssetActions.IsValid() ? AssetActions->GetTypeColor() : FColor::White);
				}
			}
		}

		HorizBox->AddSlot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SExpanderArrow, SharedThis(this))
			];

		if (IconWidget.IsValid())
		{
			HorizBox->AddSlot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(0, 0, 4, 0)
				[
					IconWidget.ToSharedRef()
				];
		}

		HorizBox->AddSlot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				GenerateWidget()
			];
	}
	else if(ColumnName == NAME_ColumnType)
	{
		if (BrowserObject->GetType() == ESearchNodeType::Asset)
		{
			TSharedPtr<FAssetNode> AssetNode = StaticCastSharedPtr<FAssetNode>(BrowserObject);
			
			HorizBox->AddSlot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(FText::FromString(AssetNode->AssetClass))
				];
		}
	}

		////FAssetData AssetData = AssetRegistry->GetAssetByObjectPath(*BrowserObject->GetAsset);

		//// Create the thumbnail handle
		//int32 ThumbnailSizeX = 24;
		//int32 ThumbnailSizeY = 24;
		//TSharedRef<FAssetThumbnail> AssetThumbnail = MakeShared<FAssetThumbnail>(AssetData, ThumbnailSizeX, ThumbnailSizeY, ThumbnailPool);

		//return SNew(SBox)
		//	.WidthOverride(ThumbnailSizeX)
		//	.HeightOverride(ThumbnailSizeY)
		//	[
		//		AssetThumbnail->MakeThumbnailWidget()
		//	];
	
	//else if (ColumnName == CategoryClass)
	//{
	//	return SNew(STextBlock)
	//		.Text(FText::FromString(BrowserObject->GetText()));
	//}
	//else if (ColumnName == CategoryProperty)
	//{
	//	//return SNew(STextBlock)
	//	//	.Text(FText::FromString(BrowserObject->GetText()));
	//}
	//else if (ColumnName == CategoryPropertyValue)
	//{
	//	//return SNew(STextBlock)
	//	//	.Text(FText::FromString(BrowserObject->Record.Text));
	//}

	return HorizBox;
}

FReply SSearchTreeRow::OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	if (BrowserObject->GetType() == ESearchNodeType::Asset)
	{
		TSharedPtr<FAssetNode> ObjectNode = StaticCastSharedPtr<FAssetNode>(BrowserObject);

		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ObjectNode->AssetPath);
	}
	else if (BrowserObject->GetType() == ESearchNodeType::Object || BrowserObject->GetType() == ESearchNodeType::Property)
	{
		FSoftObjectPath ReferencePath(BrowserObject->GetObjectPath()); 
		UObject* Object = ReferencePath.TryLoad();
		
		if (Object && Object->GetTypedOuter<UBlueprint>())  
		{
			FKismetEditorUtilities::BringKismetToFocusAttentionOnObject(Object, false); 
		}

		if (Object && (Object->GetTypedOuter<UMaterial>() || Object->GetTypedOuter<UMaterialFunction>()))
		{
			const FString& AssetPathName = ReferencePath.GetAssetPathString();
			UPackage* Package = LoadPackage(NULL, *AssetPathName, LOAD_NoRedirects);

			if (Package)
			{
				Package->FullyLoad();

				FString AssetName = FPaths::GetBaseFilename(AssetPathName);
				UObject* MaterialObject = FindObject<UObject>(Package, *AssetName);

				if (MaterialObject != NULL)
				{
					UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();
					AssetEditorSubsystem->OpenEditorForAsset(MaterialObject);

					if (IMaterialEditor* MaterialEditor = (IMaterialEditor*)(AssetEditorSubsystem->FindEditorForAsset(MaterialObject, true)))
					{
						MaterialEditor->JumpToExpression(Cast<UMaterialExpression>(Object));
					}
				}
			}
		}
		else
		{
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(ReferencePath.GetAssetPathString());
		}
	}

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
