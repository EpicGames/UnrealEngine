// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/Views/SListView.h"

class FAssetThumbnailPool;

#define LOCTEXT_NAMESPACE "SObjectBrowserTableRow"

class FSearchNode;
class IAssetRegistry;

/**
 * Implements a row widget for a live uobject.
 */
class SSearchTreeRow : public SMultiColumnTableRow<TSharedPtr<FSearchNode>>
{
public:

	SLATE_BEGIN_ARGS(SSearchTreeRow) { }
		SLATE_ARGUMENT(TSharedPtr<FSearchNode>, Object)
		SLATE_ARGUMENT(FText, HighlightText)
	SLATE_END_ARGS()

public:
	static FName NAME_ColumnName;
	static FName NAME_ColumnType;
	//static FName CategoryProperty;
	//static FName CategoryPropertyValue;

public:

	SSearchTreeRow()
	{
	}

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView, IAssetRegistry* InAssetRegistry, TSharedPtr<FAssetThumbnailPool> InThumbnailPool);

public:

	virtual TSharedRef<SWidget> GenerateWidget();

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;
	
	virtual FReply OnMouseButtonDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent) override;

private:

	TSharedPtr<FSearchNode> BrowserObject;

	IAssetRegistry* AssetRegistry;

	TSharedPtr<FAssetThumbnailPool> ThumbnailPool;
};


#undef LOCTEXT_NAMESPACE
