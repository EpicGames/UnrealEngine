// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SearchQuery.h"

enum class ESearchNodeType : uint8
{
	Asset,
	Object,
	Property
};

class FSearchNode : public TSharedFromThis<FSearchNode>
{
public:
	virtual ~FSearchNode() { }
	virtual void GetChildren(TArray<TSharedPtr<FSearchNode>>& OutChildren) { }
	virtual ESearchNodeType GetType() const = 0;
	virtual FString GetText() const = 0;
	virtual FString GetObjectPath() const = 0;

	float GetTotalScore() const { return TotalScore; }
	float GetMaxScore() const { return MaxScore; }

protected:
	float TotalScore = 0;
	float MaxScore = 0;
};

class FAssetObjectPropertyNode : public FSearchNode
{
public:
	FAssetObjectPropertyNode(const FSearchRecord& InResult)
		: Record(InResult)
	{
	}

	virtual ESearchNodeType GetType() const override { return ESearchNodeType::Property; }
	virtual FString GetText() const override
	{
		return Record.property_name + TEXT(" = ") +  Record.value_text.Replace(TEXT("\n"), TEXT(" "));
	}

	virtual FString GetObjectPath() const override
	{
		return Record.object_path;
	}

	FSearchRecord Record;
};

class FAssetObjectNode : public FSearchNode
{
public:
	FAssetObjectNode(const FSearchRecord& InResult)
		: object_name(InResult.object_name)
		, object_path(InResult.object_path)
		, object_native_class(InResult.object_native_class)
	{
		Append(InResult);
	}

	void Append(const FSearchRecord& InResult)
	{
		Properties.Add(MakeShared<FAssetObjectPropertyNode>(InResult));
		TotalScore += InResult.Score;
		MaxScore = FMath::Min(MaxScore, InResult.Score);
	}

	virtual ESearchNodeType GetType() const override { return ESearchNodeType::Object; }

	virtual void GetChildren(TArray<TSharedPtr<FSearchNode>>& OutChildren) override
	{
		for (TSharedPtr<FAssetObjectPropertyNode>& Property : Properties)
		{
			OutChildren.Add(Property);
		}
	}

	virtual FString GetText() const override
	{
		return object_name;
	}

	virtual FString GetObjectPath() const override
	{
		return object_path;
	}

	FString object_name;
	FString object_path;
	FString object_native_class;

	TArray<TSharedPtr<FAssetObjectPropertyNode>> Properties;
};

class FAssetNode : public FSearchNode
{
public:
	FAssetNode(const FSearchRecord& InResult)
		: AssetName(InResult.AssetName)
		, AssetPath(InResult.AssetPath)
		, AssetClass(InResult.AssetClass.ToString())
	{
		Append(InResult);
	}

	void Append(const FSearchRecord& InResult)
	{
		if (InResult.object_path == AssetPath)
		{
			Properties.Add(MakeShared<FAssetObjectPropertyNode>(InResult));
			TotalScore += InResult.Score;
			MaxScore = FMath::Min(MaxScore, InResult.Score);
			return;
		}

		if (!InResult.object_path.IsEmpty())
		{
			TSharedPtr<FAssetObjectNode> ExistingObject = Objects.FindRef(InResult.object_path);
			if (!ExistingObject.IsValid())
			{
				ExistingObject = MakeShared<FAssetObjectNode>(InResult);
				Objects.Add(InResult.object_path, ExistingObject);
			}
			else
			{
				ExistingObject->Append(InResult);
			}
			
			SortedObjectArray.Reset();
			Objects.GenerateValueArray(SortedObjectArray);
		}

		TotalScore += InResult.Score;
		MaxScore = FMath::Min(MaxScore, InResult.Score);
	}

	virtual ESearchNodeType GetType() const override { return ESearchNodeType::Asset; }

	virtual void GetChildren(TArray<TSharedPtr<FSearchNode>>& OutChildren) override
	{
		for (TSharedPtr<FAssetObjectPropertyNode>& Property : Properties)
		{
			OutChildren.Add(Property);
		}

		for (TSharedPtr<FAssetObjectNode>& Object : SortedObjectArray)
		{
			OutChildren.Add(Object);
		}
	}

	virtual FString GetText() const override
	{
		return AssetName;
	}

	virtual FString GetObjectPath() const override
	{
		return AssetPath;
	}

	FString AssetName;
	FString AssetPath;
	FString AssetClass;

	TArray<TSharedPtr<FAssetObjectPropertyNode>> Properties;
	TMap<FString, TSharedPtr<FAssetObjectNode>> Objects;

	TArray<TSharedPtr<FAssetObjectNode>> SortedObjectArray;
};
