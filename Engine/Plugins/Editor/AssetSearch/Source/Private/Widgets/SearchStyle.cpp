// Copyright Epic Games, Inc. All Rights Reserved.

#include "SearchStyle.h"

#include "Brushes/SlateImageBrush.h"
#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "Styling/SlateStyleRegistry.h"

FPluginStyleSet::FPluginStyleSet(const FName& InPluginName, const FName& InStyleSetName)
	: FSlateStyleSet(InStyleSetName == NAME_None ? InPluginName : FName(*(InPluginName.ToString() + TEXT(".") + InStyleSetName.ToString())))
{
	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(InPluginName.ToString());
	check(Plugin.IsValid());

	SetContentRoot(Plugin->GetContentDir());
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));
}

const FVector2D Icon16x16(16.0f, 16.0f);
const FVector2D Icon20x20(20.0f, 20.0f);
const FVector2D Icon40x40(40.0f, 40.0f);

FSearchStyle::FSearchStyle()
	: FPluginStyleSet(TEXT("AssetSearch"))
{
	const FVector2D IconSize(20.0f, 20.0f);
	const FVector2D SmallIconSize(20.0f, 20.0f);
	const FVector2D LabelIconSize(16.0f, 16.0f);

	Set("Stats", new FSlateImageBrush(RootToContentDir(TEXT("icon_tab_Stats_16x.png")), Icon16x16));

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FSearchStyle::~FSearchStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FSearchStyle& FSearchStyle::Get()
{
	static FSearchStyle Inst;
	return Inst;
}
