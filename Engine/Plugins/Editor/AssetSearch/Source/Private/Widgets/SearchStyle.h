// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

/**
 * 
 */
class FPluginStyleSet : public FSlateStyleSet
{
public:
	FPluginStyleSet(const FName& InPluginName, const FName& InStyleSetName = NAME_None);
};

/**
 * 
 */
class FSearchStyle : public FPluginStyleSet
{
public:

	/** Access the singleton instance for this style set */
	static FSearchStyle& Get();

private:

	FSearchStyle();
	~FSearchStyle();
};
