// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Framework/Commands/Commands.h"

class FAssetSearchCommands : public TCommands<FAssetSearchCommands>
{
public:
	FAssetSearchCommands();

	// TCommands<> interface
	virtual void RegisterCommands() override;
	// End of TCommands<> interface
	
	TSharedPtr<FUICommandInfo> ViewAssetSearch;
};
