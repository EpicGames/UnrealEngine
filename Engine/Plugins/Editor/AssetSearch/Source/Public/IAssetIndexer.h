// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

class UObject;
class FSearchSerializer;

class IAssetIndexer
{
public:

	virtual ~IAssetIndexer() = 0;
	virtual FString GetName() const = 0;
	virtual int32 GetVersion() const = 0;
	virtual void IndexAsset(const UObject* InAssetObject, FSearchSerializer& Serializer) const = 0;

	/**
	 * If your package contains a nested asset, such as the Blueprint stored in Level/World packages, 
	 * it would return UBlueprint's class in the array.  This is only important if you use IndexNestedAsset.
	 */
	virtual void GetNestedAssetTypes(TArray<UClass*>& OutTypes) const { }
};

inline IAssetIndexer::~IAssetIndexer() = default;
