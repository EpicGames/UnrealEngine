// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "SearchQuery.h"

class UClass;
class IAssetIndexer;
class ISearchProvider;

struct FSearchStats
{
	int32 Scanning = 0;
	int32 Processing = 0;
	int32 Updating = 0;

	int32 AssetsMissingIndex = 0;

	int64 TotalRecords = 0;

	bool IsUpdating() const
	{
		return (Scanning + Processing + Updating) > 0;
	}
};

/**
 *
 */
class IAssetSearchModule : public IModuleInterface
{
public:
	static inline IAssetSearchModule& Get()
	{
		static const FName ModuleName = "AssetSearch";
		return FModuleManager::LoadModuleChecked<IAssetSearchModule>(ModuleName);
	}

	static inline bool IsAvailable()
	{
		static const FName ModuleName = "AssetSearch";
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

	virtual FSearchStats GetStats() const = 0;

	virtual void Search(FSearchQueryPtr SearchQuery) = 0;

	virtual void ForceIndexOnAssetsMissingIndex() = 0;

	virtual void RegisterAssetIndexer(const UClass* AssetClass, TUniquePtr<IAssetIndexer>&& Indexer) = 0;
	virtual void RegisterSearchProvider(FName SearchProviderName, TUniquePtr<ISearchProvider>&& InSearchProvider) = 0;

public:

	/** Virtual destructor. */
	virtual ~IAssetSearchModule() { }
};
