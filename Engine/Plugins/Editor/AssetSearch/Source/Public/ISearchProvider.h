// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SearchQuery.h"

class ISearchProvider
{
public:
	virtual ~ISearchProvider() { }
	virtual void Search(FSearchQueryPtr SearchQuery) = 0;
};
