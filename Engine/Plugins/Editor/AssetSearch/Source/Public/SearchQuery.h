// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/ScopeLock.h"
#include "Templates/SharedPointer.h"
#include "UObject/TopLevelAssetPath.h"

struct FSearchRecord
{
	FString AssetName;
	FString AssetPath;
	FTopLevelAssetPath AssetClass;

	FString object_name;
	FString object_path;
	FString object_native_class;

	FString property_name;
	FString property_field;
	FString property_class;

	FString value_text;
	FString value_hidden;

	float Score;
};

class FSearchQuery : public TSharedFromThis<FSearchQuery, ESPMode::ThreadSafe>
{
public:
	typedef TFunction<void(TArray<FSearchRecord>&&)> ResultsCallbackFunction;

	FSearchQuery(FString InQueryText)
		: QueryText(InQueryText)
	{
	}

	const FString QueryText;

	void SetResultsCallback(ResultsCallbackFunction InCallback)
	{
		FScopeLock ScopeLock(&CallbackCS);
		Callback = InCallback;
	}

	void ClearResultsCallback()
	{
		FScopeLock ScopeLock(&CallbackCS);
		Callback = ResultsCallbackFunction();
	}

	bool IsQueryStillImportant() const
	{
		return GetResultsCallback() ? true : false;
	}

	ResultsCallbackFunction GetResultsCallback() const
	{
		ResultsCallbackFunction LocalCallback;
		{
			FScopeLock ScopeLock(&CallbackCS);
			LocalCallback = Callback;
		}
		return LocalCallback;
	}

private:
	ResultsCallbackFunction Callback;
	mutable FCriticalSection CallbackCS;
};

typedef TSharedPtr<FSearchQuery, ESPMode::ThreadSafe> FSearchQueryPtr;
typedef TWeakPtr<FSearchQuery, ESPMode::ThreadSafe> FSearchQueryWeakPtr;
