// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Serialization/JsonWriter.h"
#include "AssetRegistry/AssetData.h"

template <class CharType> struct TCondensedJsonPrintPolicy;

class IAssetIndexer;

class FSearchSerializer : public FNoncopyable
{
public:
	FSearchSerializer(const FAssetData& InAsset, FArchive& InAr);
	FSearchSerializer(const FAssetData& InAsset, FString* const Stream);
	~FSearchSerializer();

	static int32 GetVersion();

	bool IndexAsset(const UObject* InAsset, const TMap<FName, TUniquePtr<IAssetIndexer>>& Indexers);

	void BeginIndexer(const IAssetIndexer* InIndexer);

	//---------------------------------------------------------------

	ASSETSEARCH_API void BeginIndexingObject(const UObject* InObject, const FString& InFriendlyName);
	ASSETSEARCH_API void BeginIndexingObject(const UObject* InObject, const FText& InFriendlyName);
	ASSETSEARCH_API void EndIndexingObject();

	//---------------------------------------------------------------

	ASSETSEARCH_API void IndexProperty(const FString& InName, const FName& InValue);
	ASSETSEARCH_API void IndexProperty(const FString& InName, const FText& InValue);
	ASSETSEARCH_API void IndexProperty(const FString& InName, const FString& InValue);
	ASSETSEARCH_API void IndexProperty(const UClass* InPropertyClass, const FString& InName, const FString& InValue);
	ASSETSEARCH_API void IndexProperty(const UClass* InPropertyClass, const FString& InName, const FText& InValue);
	ASSETSEARCH_API void IndexProperty(const FProperty* InProperty, const FString& InValue);

	//---------------------------------------------------------------

	ASSETSEARCH_API void IndexNestedAsset(UObject* InNestedAsset);

	//---------------------------------------------------------------

	void EndIndexer();

private:
	void IndexProperty(const FFieldClass& InPropertyFieldClass, const UClass* PropertyClass, const FString& InName, const FString& InValue);

private:
	void Initialize(const FAssetData& InAsset);

private:
	TSharedRef< TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR> > > JsonWriter;

	FAssetData AssetData;

	const IAssetIndexer* CurrentIndexer = nullptr;

	const UObject* CurrentObject = nullptr;
	FString CurrentObjectName;
	FString CurrentObjectClass;
	FString CurrentObjectPath;

	struct FIndexedValue
	{
		FString PropertyName;
		FString PropertyFieldClass;
		FName PropertyClass;

		FString Text;
		FString HiddenText;
	};

	TArray<FIndexedValue> Values;

	TArray<UObject*> NestedAssets;
};
