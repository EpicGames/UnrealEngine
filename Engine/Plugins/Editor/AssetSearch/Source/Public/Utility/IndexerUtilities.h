// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#define UE_API ASSETSEARCH_API

class FProperty;
class FString;
class UObject;
class UStruct;
template <typename FuncType> class TFunctionRef;

class FIndexerUtilities
{
public:
	static UE_API void IterateIndexableProperties(const UObject* InObject, TFunctionRef<void(const FProperty* /*Property*/, const FString& /*Value*/)> Callback);
	static UE_API void IterateIndexableProperties(const UStruct* InStruct, const void* InStructValue, TFunctionRef<void(const FProperty* /*Property*/, const FString& /*Value*/)> Callback);
};

#undef UE_API
